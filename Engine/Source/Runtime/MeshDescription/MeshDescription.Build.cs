// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MeshDescription : ModuleRules
	{
		public MeshDescription(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject"
				}
			);

			if (Target.bBuildWithEditorOnlyData && Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop))
			{
				PrivateDependencyModuleNames.Add("DerivedDataCache");
			}
		}
	}
}
