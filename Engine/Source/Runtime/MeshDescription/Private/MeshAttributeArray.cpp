// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshAttributeArray.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"


FArchive& operator<<(FArchive& Ar, FAttributesSetEntry& Entry)
{
	if (Ar.IsLoading())
	{
		uint32 AttributeType;
		Ar << AttributeType;

		uint32 Extent = 1;
		if (Ar.CustomVer(FReleaseObjectVersion::GUID) == FReleaseObjectVersion::MeshDescriptionNewFormat ||
			Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::MeshDescriptionNewFormat)
		{
			Ar << Extent;
		}

		Entry.CreateArrayOfType(AttributeType, Extent);
		Entry.Ptr->Serialize(Ar);
	}
	else
	{
		check(Entry.Ptr.IsValid());
		uint32 AttributeType = Entry.Ptr->GetType();
		Ar << AttributeType;
		uint32 Extent = Entry.Ptr->GetExtent();
		Ar << Extent;
		Entry.Ptr->Serialize(Ar);
	}

	return Ar;
}


template <typename T>
void SerializeLegacy( FArchive& Ar, FAttributesSetBase& AttributesSet )
{
	Ar << AttributesSet.NumElements;

	TMap<FName, TMeshAttributeArraySet<T>> OldContainer;
	Ar << OldContainer;

	for( const auto& MapEntry : OldContainer )
	{
		AttributesSet.RegisterAttribute<T>( MapEntry.Key, 0 );
		static_cast<TMeshAttributeArraySet<T>&>( *AttributesSet.Map.FindChecked( MapEntry.Key ).Get() ) = MapEntry.Value;
	}
}


FArchive& operator<<( FArchive& Ar, FAttributesSetBase& AttributesSet )
{
	Ar.UsingCustomVersion( FEditorObjectVersion::GUID );
	Ar.UsingCustomVersion( FReleaseObjectVersion::GUID );

	if( Ar.IsLoading() && Ar.CustomVer( FEditorObjectVersion::GUID ) < FEditorObjectVersion::MeshDescriptionNewAttributeFormat )
	{
		// Legacy serialization format
		int32 NumAttributeTypes;
		Ar << NumAttributeTypes;
		check( NumAttributeTypes == 7 );

		AttributesSet.Map.Empty();
		SerializeLegacy<FVector4f>( Ar, AttributesSet );
		SerializeLegacy<FVector3f>( Ar, AttributesSet );
		SerializeLegacy<FVector2f>( Ar, AttributesSet );
		SerializeLegacy<float>( Ar, AttributesSet );
		SerializeLegacy<int>( Ar, AttributesSet );
		SerializeLegacy<bool>( Ar, AttributesSet );
		SerializeLegacy<FName>( Ar, AttributesSet );

		return Ar;
	}

	Ar << AttributesSet.NumElements;

	// If saving, store transient attribute arrays and remove them temporarily from the map
	TArray<TTuple<FName, FAttributesSetEntry>> TransientArrays;
	if( Ar.IsSaving() && !Ar.IsTransacting() )
	{
		for( TMap<FName, FAttributesSetEntry>::TIterator It( AttributesSet.Map ); It; ++It )
		{
			if( EnumHasAnyFlags( It.Value()->GetFlags(), EMeshAttributeFlags::Transient ) )
			{
				TransientArrays.Emplace( MakeTuple( It.Key(), MoveTemp( It.Value() ) ) );
			}
		}
		
		// Remove the items outside the for loop above, since compact sets disallow removal via an iterator during iteration.
		for (const TTuple<FName, FAttributesSetEntry>& Item: TransientArrays)
		{
			AttributesSet.Map.Remove(Item.Key);
		}
	}

	// Serialize map
	Ar << AttributesSet.Map;

	// Restore transient attribute arrays if saving
	if( Ar.IsSaving() && !Ar.IsTransacting() )
	{
		for( TTuple<FName, FAttributesSetEntry>& TransientArray : TransientArrays )
		{
			AttributesSet.Map.Emplace( TransientArray.Get<0>(), MoveTemp( TransientArray.Get<1>() ) );
		}
	}

	return Ar;
}


void FAttributesSetBase::Remap( const TSparseArray<int32>& IndexRemap )
{
	// Determine the number of elements by finding the maximum remapped element index in the IndexRemap array.
	NumElements = 0;
	for( const int32 ElementIndex : IndexRemap )
	{
		NumElements = FMath::Max( NumElements, ElementIndex + 1 );
	}

	for( auto& MapEntry : Map )
	{
		if (MapEntry.Value->GetNumChannels() > 0)
		{
			MapEntry.Value->Remap( IndexRemap );
			check( MapEntry.Value->GetNumElements() == NumElements );
		}
	}
}


void FAttributesSetBase::AppendAttributesFrom(const FAttributesSetBase& OtherAttributesSet)
{
	check(OtherAttributesSet.NumElements == NumElements);

	for (const auto& MapPair : OtherAttributesSet.Map)
	{
		if (Map.Contains(MapPair.Key))
		{
			UE_LOGF(LogMeshDescription, Log, "Appending attribute '%ls' which already exists.", *MapPair.Key.ToString());
		}
		else
		{
			Map.Add(MapPair.Key, MapPair.Value);
		}
	}
}
