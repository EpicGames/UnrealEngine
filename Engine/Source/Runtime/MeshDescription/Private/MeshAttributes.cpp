// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshAttributes.h"

namespace MeshAttribute
{
	namespace Vertex
	{
		const FName Position("Position ");
	}

	namespace VertexInstance
	{
		const FName VertexIndex("VertexIndex");
	}

	namespace Edge
	{
		const FName VertexIndex("VertexIndex");
	}

	namespace Triangle
	{
		const FName VertexInstanceIndex("VertexInstanceIndex");
		const FName PolygonIndex("PolygonIndex");
		const FName EdgeIndex("EdgeIndex");
		const FName VertexIndex("VertexIndex");
		const FName UVIndex("UVIndex");
		const FName PolygonGroupIndex("PolygonGroupIndex");
	}

	namespace UV
	{
		const FName UVCoordinate("UVCoordinate");
	}

	namespace Polygon
	{
		const FName PolygonGroupIndex("PolygonGroupIndex");
	}
}


void FMeshAttributes::Register(bool bKeepExistingAttribute)
{
	// Nothing to do here: Vertex positions are already registered by the FMeshDescription constructor
}

