// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshDescription.h"

#include "Algo/Copy.h"
#include "MeshAttributes.h"
#include "Misc/SecureHash.h"
#include "Serialization/BulkData.h"
#include "Serialization/EditorBulkDataReader.h"
#include "Serialization/EditorBulkDataWriter.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/NameAsStringProxyArchive.h"
#include "UObject/EnterpriseObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshDescription)

#if WITH_EDITORONLY_DATA
#include "DerivedDataBuildVersion.h"
#endif

#if WITH_EDITOR
#include "Misc/ScopeRWLock.h"
#endif

FName FMeshDescription::VerticesName("Vertices");
FName FMeshDescription::VertexInstancesName("VertexInstances");
FName FMeshDescription::UVsName("UVs");
FName FMeshDescription::EdgesName("Edges");
FName FMeshDescription::TrianglesName("Triangles");
FName FMeshDescription::PolygonsName("Polygons");
FName FMeshDescription::PolygonGroupsName("PolygonGroups");


FMeshDescription::FMeshDescription()
{
	Initialize();
}


FMeshDescription::FMeshDescription(const FMeshDescription& Other)
{
	Elements = Other.Elements;
	Cache();
}


FMeshDescription& FMeshDescription::operator=(const FMeshDescription& Other)
{
	if (this != &Other)
	{
		Elements = Other.Elements;
		Cache();
	}
	return *this;
}


void FMeshDescription::Initialize()
{
	// Register the basic mesh element types
	VertexElements = Elements.Emplace(VerticesName).Get();
	VertexInstanceElements = Elements.Emplace(VertexInstancesName).Get();
	UVElements = Elements.Emplace(UVsName).Get();
	EdgeElements = Elements.Emplace(EdgesName).Get();
	TriangleElements = Elements.Emplace(TrianglesName).Get();
	PolygonElements = Elements.Emplace(PolygonsName).Get();
	PolygonGroupElements = Elements.Emplace(PolygonGroupsName).Get();

	// Now register topology-based attributes

	// Register vertex reference for vertex instances
	VertexInstanceVertices = VertexInstanceElements->Get().GetAttributes().RegisterIndexAttribute<FVertexID>(MeshAttribute::VertexInstance::VertexIndex, 1, EMeshAttributeFlags::Mandatory);

	// Register vertex references for edges
	EdgeVertices = EdgeElements->Get().GetAttributes().RegisterIndexAttribute<FVertexID[2]>(MeshAttribute::Edge::VertexIndex, 1, EMeshAttributeFlags::Mandatory);

	// Register vertex instance and polygon references for triangles
	TriangleVertexInstances = TriangleElements->Get().GetAttributes().RegisterIndexAttribute<FVertexInstanceID[3]>(MeshAttribute::Triangle::VertexInstanceIndex, 1, EMeshAttributeFlags::Mandatory);
	TrianglePolygons = TriangleElements->Get().GetAttributes().RegisterIndexAttribute<FPolygonID>(MeshAttribute::Triangle::PolygonIndex, 1, EMeshAttributeFlags::Mandatory);
	TrianglePolygonGroups = TriangleElements->Get().GetAttributes().RegisterIndexAttribute<FPolygonGroupID>(MeshAttribute::Triangle::PolygonGroupIndex, 1, EMeshAttributeFlags::Mandatory);

	// Register UV references for triangles
	TriangleUVs = TriangleElements->Get().GetAttributes().RegisterIndexAttribute<FUVID[3]>(MeshAttribute::Triangle::UVIndex, 0, EMeshAttributeFlags::Mandatory);

	// Register vertex and edge references for triangles; these are transient references which are generated at load time
	TriangleVertices = TriangleElements->Get().GetAttributes().RegisterIndexAttribute<FVertexID[3]>(MeshAttribute::Triangle::VertexIndex, 1, EMeshAttributeFlags::Mandatory | EMeshAttributeFlags::Transient);
	TriangleEdges = TriangleElements->Get().GetAttributes().RegisterIndexAttribute<FEdgeID[3]>(MeshAttribute::Triangle::EdgeIndex, 1, EMeshAttributeFlags::Mandatory | EMeshAttributeFlags::Transient);

	// Register polygon group reference for polygons
	PolygonPolygonGroups = PolygonElements->Get().GetAttributes().RegisterIndexAttribute<FPolygonGroupID>(MeshAttribute::Polygon::PolygonGroupIndex, 1, EMeshAttributeFlags::Mandatory);

	// Minimal requirement is that vertices have a Position attribute
	VertexPositions = VertexElements->Get().GetAttributes().RegisterAttribute(MeshAttribute::Vertex::Position, 1, FVector3f::ZeroVector, EMeshAttributeFlags::Lerpable | EMeshAttributeFlags::Mandatory);

	// Register UVCoordinates attribute for UVs
	UVElements->Get().GetAttributes().RegisterAttribute(MeshAttribute::UV::UVCoordinate, 1, FVector2f::ZeroVector, EMeshAttributeFlags::Lerpable | EMeshAttributeFlags::Mandatory);

	// Associate indexers with element types and their referencing attributes
	InitializeIndexers();
}


void FMeshDescription::InitializeIndexers()
{
	// Vertices may typically have 6 vertex instances from the triangles which include them
	VertexToVertexInstances.SetInitialNumReferences(6);
	VertexToVertexInstances.Set(VertexElements, VertexInstanceElements, MeshAttribute::VertexInstance::VertexIndex);

	// Vertices may typically have 6 edges from the triangles which include them
	VertexToEdges.SetInitialNumReferences(6);
	VertexToEdges.Set(VertexElements, EdgeElements, MeshAttribute::Edge::VertexIndex);

	// @todo: VertexToTriangles?
	VertexInstanceToTriangles.Set(VertexInstanceElements, TriangleElements, MeshAttribute::Triangle::VertexInstanceIndex);

	// Assume most edges will have either 1 or 2 triangles
	EdgeToTriangles.SetInitialNumReferences(2);
	EdgeToTriangles.Set(EdgeElements, TriangleElements, MeshAttribute::Triangle::EdgeIndex);

	// UVs may typically be used by 6 adjacent triangles
	UVToTriangles.SetInitialNumReferences(6);
	UVToTriangles.Set(UVElements, TriangleElements, MeshAttribute::Triangle::UVIndex);

	// Assume most polygons are composed of only 1 triangle
	PolygonToTriangles.SetInitialNumReferences(1);
	PolygonToTriangles.Set(PolygonElements, TriangleElements, MeshAttribute::Triangle::PolygonIndex);

	// Polygon group indexers are a little different; in general there will be many back references and few keys, so set not to use chunks.
	PolygonGroupToTriangles.SetUnchunked();
	PolygonGroupToPolygons.SetUnchunked();
	PolygonGroupToTriangles.SetInitialNumReferences(256);
	PolygonGroupToPolygons.SetInitialNumReferences(256);
	PolygonGroupToTriangles.Set(PolygonGroupElements, TriangleElements, MeshAttribute::Triangle::PolygonGroupIndex);
	PolygonGroupToPolygons.Set(PolygonGroupElements, PolygonElements, MeshAttribute::Polygon::PolygonGroupIndex);
}


void FMeshDescription::Cache()
{
	// Get pointers to element containers
	VertexElements = Elements.Find(VerticesName)->Get();
	VertexInstanceElements = Elements.Find(VertexInstancesName)->Get();
	UVElements = Elements.Find(UVsName)->Get();
	EdgeElements = Elements.Find(EdgesName)->Get();
	TriangleElements = Elements.Find(TrianglesName)->Get();
	PolygonElements = Elements.Find(PolygonsName)->Get();
	PolygonGroupElements = Elements.Find(PolygonGroupsName)->Get();

	// Register required transient attributes
	TriangleVertices = TriangleElements->Get().GetAttributes().RegisterIndexAttribute<FVertexID[3]>(MeshAttribute::Triangle::VertexIndex, 1, EMeshAttributeFlags::Mandatory | EMeshAttributeFlags::Transient);
	TriangleEdges = TriangleElements->Get().GetAttributes().RegisterIndexAttribute<FEdgeID[3]>(MeshAttribute::Triangle::EdgeIndex, 1, EMeshAttributeFlags::Mandatory | EMeshAttributeFlags::Transient);

	// Cache fundamental attribute arrays
	VertexInstanceVertices = VertexInstanceElements->Get().GetAttributes().GetAttributesRef<FVertexID>(MeshAttribute::VertexInstance::VertexIndex);
	EdgeVertices = EdgeElements->Get().GetAttributes().GetAttributesRef<TArrayView<FVertexID>>(MeshAttribute::Edge::VertexIndex);
	TriangleVertexInstances = TriangleElements->Get().GetAttributes().GetAttributesRef<TArrayView<FVertexInstanceID>>(MeshAttribute::Triangle::VertexInstanceIndex);
	TrianglePolygons = TriangleElements->Get().GetAttributes().GetAttributesRef<FPolygonID>(MeshAttribute::Triangle::PolygonIndex);
	TrianglePolygonGroups = TriangleElements->Get().GetAttributes().GetAttributesRef<FPolygonGroupID>(MeshAttribute::Triangle::PolygonGroupIndex);
	TriangleUVs = TriangleElements->Get().GetAttributes().GetAttributesRef<TArrayView<FUVID>>(MeshAttribute::Triangle::UVIndex);
	PolygonPolygonGroups = PolygonElements->Get().GetAttributes().GetAttributesRef<FPolygonGroupID>(MeshAttribute::Polygon::PolygonGroupIndex);
	VertexPositions = VertexElements->Get().GetAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);

	// Associate indexers with element types and their referencing attributes
	InitializeIndexers();
}

TConstArrayView<TPair<FGuid, int32>> FMeshDescription::GetSerializationFormatVersions()
{
	// These are versions that affect the serialization of the FMeshDescription objects. Any time FMeshDescription serialization changes
	// the version should be updated to match the serialization version for the version GUID that got changed alongside it. 
	// This is to avoid the ID string changing every time the version gets updated for potentially unrelated serialization changes causing
	// unnecessary data store / DDC churn.
	static TPair<FGuid, int32> Versions[] {
			{ FEditorObjectVersion::GUID, FEditorObjectVersion::SkeletalMeshSourceDataSupport16bitOfMaterialNumber },
			{ FReleaseObjectVersion::GUID, FReleaseObjectVersion::GeometryCollectionCacheRemovesMassToLocal },
			{ FEnterpriseObjectVersion::GUID, FEnterpriseObjectVersion::AddedParametricSurfaceData },
			{ FUE5MainStreamObjectVersion::GUID, FUE5MainStreamObjectVersion::RuntimeVirtualTextureMaterialValidation },
			{ FFortniteMainBranchObjectVersion::GUID, FFortniteMainBranchObjectVersion::DynamicMeshAttributesTriangleLabels },
		};	
	return MakeArrayView(Versions); 
}


void FMeshDescription::Serialize(FArchive& BaseAr)
{
	FNameAsStringProxyArchive Ar(BaseAr);
	
	// IMPORTANT: When adding a new version index to augment the FMeshDescription serialization format,
	// please update the version listing in FMeshDescription::GetSerializationFormatVersions to either add
	// the new GUID + version or update the version for the specific CustomVersion GUID that the format
	// has been bumped up to. Otherwise, there will be issues in the various systems that rely on that
	// table to construct their DDC keys to ensure that they won't try to load wrongly versioned FMeshDescription
	// objects, whether raw or via bulk data.
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	
	if (Ar.IsLoading() && Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::MeshDescriptionNewSerialization)
	{
		UE_LOGF(LogLoad, Warning, "Deprecated serialization format");
	}

	if (Ar.IsLoading() &&
		Ar.CustomVer(FReleaseObjectVersion::GUID) != FReleaseObjectVersion::MeshDescriptionNewFormat &&
		Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::MeshDescriptionNewFormat)
	{
		// Serialize the old format data and transform it into the new format mesh element map
		SerializeLegacy(Ar);
	}
	else
	{
		Ar << Elements;

		// After loading elements, we need to re-cache mesh element and attribute arrays 
		if (Ar.IsLoading())
		{
			// Ensure there's a UV element container
			if (Elements.Find(UVsName) == nullptr)
			{
				UE_LOGF(LogLoad, Warning, "Couldn't find UV element container in mesh - adding an empty one");
				UVElements = Elements.Emplace(UVsName).Get();
			}

			Cache();

			for (FTriangleID TriangleID : TriangleElements->Get().GetElementIDs())
			{
				for (int32 I = 0; I < 3; I++)
				{
					FVertexInstanceID VertexInstanceID = TriangleVertexInstances[TriangleID][I];
					TriangleVertices[TriangleID][I] = VertexInstanceVertices[VertexInstanceID];
					VertexInstanceToTriangles.AddReferenceToKey(VertexInstanceID, TriangleID);
				}

				for (int32 I = 0; I < 3; I++)
				{
					FEdgeID EdgeID = GetVertexPairEdge(TriangleVertices[TriangleID][I], TriangleVertices[TriangleID][(I+1) % 3]);
					TriangleEdges[TriangleID][I] = EdgeID;
					EdgeToTriangles.AddReferenceToKey(EdgeID, TriangleID);
				}
			}

			RebuildIndexers();
		}
	}
}


struct FMeshVertex_Legacy
{
	TArray<FVertexInstanceID> VertexInstanceIDs;
	TArray<FEdgeID> ConnectedEdgeIDs;

	friend FArchive& operator<<(FArchive& Ar, FMeshVertex_Legacy& Vertex)
	{
		check(Ar.IsLoading());
		if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::MeshDescriptionNewSerialization)
		{
			Ar << Vertex.VertexInstanceIDs;
			Ar << Vertex.ConnectedEdgeIDs;
		}

		return Ar;
	}
};


struct FMeshVertexInstance_Legacy
{
	FVertexID VertexID;
	TArray<FTriangleID> ConnectedTriangles;

	friend FArchive& operator<<(FArchive& Ar, FMeshVertexInstance_Legacy& VertexInstance)
	{
		check(Ar.IsLoading());
		Ar << VertexInstance.VertexID;
		if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::MeshDescriptionNewSerialization)
		{
			TArray<FPolygonID> ConnectedPolygons_DISCARD;
			Ar << ConnectedPolygons_DISCARD;
		}

		return Ar;
	}
};


struct FMeshEdge_Legacy
{
	FVertexID VertexIDs[2];
	TArray<FTriangleID> ConnectedTriangles;

	friend FArchive& operator<<(FArchive& Ar, FMeshEdge_Legacy& Edge)
	{
		check(Ar.IsLoading());
		Ar << Edge.VertexIDs[0];
		Ar << Edge.VertexIDs[1];
		if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::MeshDescriptionNewSerialization)
		{
			TArray<FPolygonID> ConnectedPolygons_DISCARD;
			Ar << ConnectedPolygons_DISCARD;
		}

		return Ar;
	}
};


struct FMeshTriangle_Legacy
{
	FVertexInstanceID VertexInstanceIDs[3];
	FPolygonID PolygonID;

	friend FArchive& operator<<(FArchive& Ar, FMeshTriangle_Legacy& Triangle)
	{
		check(Ar.IsLoading());
		Ar << Triangle.VertexInstanceIDs[0];
		Ar << Triangle.VertexInstanceIDs[1];
		Ar << Triangle.VertexInstanceIDs[2];

		if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::MeshDescriptionTriangles)
		{
			Ar << Triangle.PolygonID;
		}

		return Ar;
	}
};


struct FMeshPolygon_Legacy
{
	TArray<FVertexInstanceID> VertexInstanceIDs;
	TArray<FTriangleID> TriangleIDs;
	FPolygonGroupID PolygonGroupID;

	friend FArchive& operator<<(FArchive& Ar, FMeshPolygon_Legacy& Polygon)
	{
		check(Ar.IsLoading());
		Ar << Polygon.VertexInstanceIDs;

		if (Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::MeshDescriptionRemovedHoles)
		{
			TArray<TArray<FVertexInstanceID>> Empty;
			Ar << Empty;
		}

		if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::MeshDescriptionNewSerialization)
		{
			TArray<FMeshTriangle_Legacy> Triangles_DISCARD;
			Ar << Triangles_DISCARD;
		}

		Ar << Polygon.PolygonGroupID;

		return Ar;
	}
};


struct FMeshPolygonGroup_Legacy
{
	TArray<FPolygonID> Polygons;

	friend FArchive& operator<<(FArchive& Ar, FMeshPolygonGroup_Legacy& PolygonGroup)
	{
		check(Ar.IsLoading());
		if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::MeshDescriptionNewSerialization)
		{
			Ar << PolygonGroup.Polygons;
		}

		return Ar;
	}
};


template <typename T>
struct FFixAttributesSizeHelper
{
	explicit FFixAttributesSizeHelper(int32 InExpectedNum)
		: ExpectedNum(InExpectedNum),
		  bAllDefault(true)
	{}

	template <typename U>
	void operator()(const FName AttributeName, TMeshAttributesRef<T, TArrayView<U>> AttributeArrayRef)
	{
		// Not expecting arrays in legacy attributes
		check(false);
	}

	template <typename U>
	void operator()(const FName AttributeName, TMeshAttributesRef<T, TArrayAttribute<U>> AttributeArrayRef)
	{
		// Not expecting arrays in legacy attributes
		check(false);
	}

	void operator()(const FName AttributeName, TMeshAttributesRef<T, FTransform> AttributeArrayRef)
	{
		// Not expecting FTransform in legacy attributes
		check(false);
	}

	template <typename U>
	void operator()(const FName AttributeName, TMeshAttributesRef<T, U> AttributeArrayRef)
	{
		if (bAllDefault)
		{
			for (int32 Channel = 0; Channel < AttributeArrayRef.GetNumChannels(); Channel++)
			{
				for (int32 Index = ExpectedNum; Index < AttributeArrayRef.GetNumElements(); Index++)
				{
					if (AttributeArrayRef.Get(T(Index), Channel) != AttributeArrayRef.GetDefaultValue())
					{
						bAllDefault = false;
						return;
					}
				}
			}
		}
	}

	int32 ExpectedNum;
	bool bAllDefault;
};


template <typename T>
void FixAttributesSize(int32 ExpectedNum, TAttributesSet<T>& AttributesSet)
{
	// Ensure that the attribute set is the same size as the mesh element array they describe
	// If there are extra elements, and they are not set to trivial defaults, this is an error.
	FFixAttributesSizeHelper<T> Helper(ExpectedNum);
	AttributesSet.ForEach(Helper);
	check(Helper.bAllDefault);	// If this fires, something is very wrong with the legacy asset
	AttributesSet.SetNumElements(ExpectedNum);
}

void FMeshDescription::SerializeLegacy(FArchive& Ar)
{
	TMeshElementArray<FMeshVertex_Legacy, FVertexID> VertexArray; 
	TMeshElementArray<FMeshVertexInstance_Legacy, FVertexInstanceID> VertexInstanceArray;
	TMeshElementArray<FMeshEdge_Legacy, FEdgeID> EdgeArray;
	TMeshElementArray<FMeshTriangle_Legacy, FTriangleID> TriangleArray;
	TMeshElementArray<FMeshPolygon_Legacy, FPolygonID> PolygonArray;
	TMeshElementArray<FMeshPolygonGroup_Legacy, FPolygonGroupID> PolygonGroupArray;

	TAttributesSet<FVertexID> VertexAttributesSet;
	TAttributesSet<FVertexInstanceID> VertexInstanceAttributesSet;
	TAttributesSet<FEdgeID> EdgeAttributesSet;
	TAttributesSet<FTriangleID> TriangleAttributesSet;
	TAttributesSet<FPolygonID> PolygonAttributesSet;
	TAttributesSet<FPolygonGroupID> PolygonGroupAttributesSet;

	Ar << VertexArray;
	Ar << VertexInstanceArray;
	Ar << EdgeArray;
	Ar << PolygonArray;
	Ar << PolygonGroupArray;

	Ar << VertexAttributesSet;
	Ar << VertexInstanceAttributesSet;
	Ar << EdgeAttributesSet;
	Ar << PolygonAttributesSet;
	Ar << PolygonGroupAttributesSet;

	FixAttributesSize(VertexArray.GetArraySize(), VertexAttributesSet);
	FixAttributesSize(VertexInstanceArray.GetArraySize(), VertexInstanceAttributesSet);
	FixAttributesSize(EdgeArray.GetArraySize(), EdgeAttributesSet);
	FixAttributesSize(PolygonArray.GetArraySize(), PolygonAttributesSet);
	FixAttributesSize(PolygonGroupArray.GetArraySize(), PolygonGroupAttributesSet);

	// Serialize new triangle arrays since version MeshDescriptionTriangles
	if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::MeshDescriptionTriangles)
	{
		Ar << TriangleArray;
		Ar << TriangleAttributesSet;

		FixAttributesSize(TriangleArray.GetArraySize(), TriangleAttributesSet);
	}

	// Convert the old style element arrays into the new format

	// Completely reinitialize the mesh elements map as it is not being directly serialized into.
	Initialize();

	for (FVertexID VertexID : VertexArray.GetElementIDs())
	{
		VertexElements->Get().Insert(VertexID);
	}

	for (FVertexInstanceID VertexInstanceID : VertexInstanceArray.GetElementIDs())
	{
		VertexInstanceElements->Get().Insert(VertexInstanceID);
		FVertexID VertexID = VertexInstanceArray[VertexInstanceID].VertexID;
		VertexInstanceVertices[VertexInstanceID] = VertexID;
		VertexToVertexInstances.AddReferenceToKey(VertexID, VertexInstanceID);
	}

	for (FEdgeID EdgeID : EdgeArray.GetElementIDs())
	{
		EdgeElements->Get().Insert(EdgeID);
		FVertexID VertexID0 = EdgeArray[EdgeID].VertexIDs[0];
		FVertexID VertexID1 = EdgeArray[EdgeID].VertexIDs[1];
		EdgeVertices[EdgeID][0] = VertexID0;
		EdgeVertices[EdgeID][1] = VertexID1;
		VertexToEdges.AddReferenceToKey(VertexID0, EdgeID);
		VertexToEdges.AddReferenceToKey(VertexID1, EdgeID);
	}

	for (FPolygonGroupID PolygonGroupID : PolygonGroupArray.GetElementIDs())
	{
		PolygonGroupElements->Get().Insert(PolygonGroupID);
	}

	for (FPolygonID PolygonID : PolygonArray.GetElementIDs())
	{
		PolygonElements->Get().Insert(PolygonID);
		FPolygonGroupID PolygonGroupID = PolygonArray[PolygonID].PolygonGroupID;
		PolygonPolygonGroups[PolygonID] = PolygonGroupID;
		PolygonGroupToPolygons.AddReferenceToKey(PolygonGroupID, PolygonID);

		// If the asset is pre-triangles, we generate triangles here from the polygon
		if (Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::MeshDescriptionTriangles)
		{
			check(PolygonArray[PolygonID].VertexInstanceIDs.Num() >= 3);
			CreatePolygonTriangles(PolygonID, PolygonArray[PolygonID].VertexInstanceIDs);
		}
	}

	// Only do this if there were actually triangles in the asset
	if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::MeshDescriptionTriangles)
	{
		for (FTriangleID TriangleID : TriangleArray.GetElementIDs())
		{
			TriangleElements->Get().Insert(TriangleID);
			FPolygonID PolygonID = TriangleArray[TriangleID].PolygonID;
			FPolygonGroupID PolygonGroupID = PolygonArray[PolygonID].PolygonGroupID;
			TrianglePolygons[TriangleID] = PolygonID;
			TrianglePolygonGroups[TriangleID] = PolygonGroupID;
			PolygonToTriangles.AddReferenceToKey(PolygonID, TriangleID);
			PolygonGroupToTriangles.AddReferenceToKey(PolygonGroupID, TriangleID);

			for (int32 I = 0; I < 3; I++)
			{
				FVertexInstanceID VertexInstanceID = TriangleArray[TriangleID].VertexInstanceIDs[I];
				TriangleVertexInstances[TriangleID][I] = VertexInstanceID;
				TriangleVertices[TriangleID][I] = VertexInstanceVertices[VertexInstanceID];
				VertexInstanceToTriangles.AddReferenceToKey(VertexInstanceID, TriangleID);
			}

			for (int32 I = 0; I < 3; I++)
			{
				FEdgeID EdgeID = GetVertexPairEdge(TriangleVertices[TriangleID][I], TriangleVertices[TriangleID][(I+1) % 3]);
				TriangleEdges[TriangleID][I] = EdgeID;
				EdgeToTriangles.AddReferenceToKey(EdgeID, TriangleID);
			}
		}
	}

	// Unregister the position attribute from the vertex elements, as it will come from the legacy data instead.
	VertexElements->Get().GetAttributes().UnregisterAttribute(MeshAttribute::Vertex::Position);

	// Add the legacy mesh attributes into the new containers
	VertexElements->Get().GetAttributes().AppendAttributesFrom(VertexAttributesSet);
	VertexInstanceElements->Get().GetAttributes().AppendAttributesFrom(VertexInstanceAttributesSet);
	EdgeElements->Get().GetAttributes().AppendAttributesFrom(EdgeAttributesSet);
	PolygonElements->Get().GetAttributes().AppendAttributesFrom(PolygonAttributesSet);
	PolygonGroupElements->Get().GetAttributes().AppendAttributesFrom(PolygonGroupAttributesSet);

	if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::MeshDescriptionTriangles)
	{
		TriangleElements->Get().GetAttributes().AppendAttributesFrom(TriangleAttributesSet);
	}

	Cache();
	BuildIndexers();
}


void FMeshDescription::ResetIndexers()
{
	VertexToVertexInstances.Reset();
	VertexToEdges.Reset();
	VertexInstanceToTriangles.Reset();
	EdgeToTriangles.Reset();
	UVToTriangles.Reset();
	PolygonToTriangles.Reset();
	PolygonGroupToTriangles.Reset();
	PolygonGroupToPolygons.Reset();
}


void FMeshDescription::BuildIndexers()
{
	VertexToVertexInstances.Build();
	VertexToEdges.Build();
	VertexInstanceToTriangles.Build();
	EdgeToTriangles.Build();
	UVToTriangles.Build();
	PolygonToTriangles.Build();
	PolygonGroupToTriangles.Build();
	PolygonGroupToPolygons.Build();
}


void FMeshDescription::RebuildIndexers()
{
	VertexToVertexInstances.ForceRebuild();
	VertexToEdges.ForceRebuild();
	VertexInstanceToTriangles.ForceRebuild();
	EdgeToTriangles.ForceRebuild();
	UVToTriangles.ForceRebuild();
	PolygonToTriangles.ForceRebuild();
	PolygonGroupToTriangles.ForceRebuild();
	PolygonGroupToPolygons.ForceRebuild();
}


void FMeshDescription::Empty()
{
	for (TPair<FName, FMeshElementTypeWrapper>& ElementsItem: Elements)
	{
		ElementsItem.Value.Get()->Reset();
	}
	ResetIndexers();
}


bool FMeshDescription::IsEmpty() const
{
	for (const TPair<FName, FMeshElementTypeWrapper>& ElementsItem: Elements)
	{
		if (!ElementsItem.Value.Get()->IsEmpty())
		{
			return false;
		}
	}
	return true;
}

bool FMeshDescription::NeedsCompact() const
{
	return
	(
		VertexElements->Get().GetArraySize()			!= VertexElements->Get().Num()			||
		VertexInstanceElements->Get().GetArraySize()	!= VertexInstanceElements->Get().Num()	||
		EdgeElements->Get().GetArraySize()				!= EdgeElements->Get().Num()			||
		TriangleElements->Get().GetArraySize()			!= TriangleElements->Get().Num()		||
		PolygonElements->Get().GetArraySize()			!= PolygonElements->Get().Num()			||
		PolygonGroupElements->Get().GetArraySize()		!= PolygonGroupElements->Get().Num()
	);
}

void FMeshDescription::Compact(FElementIDRemappings& OutRemappings)
{
	VertexElements->Get().Compact(OutRemappings.NewVertexIndexLookup);
	VertexInstanceElements->Get().Compact(OutRemappings.NewVertexInstanceIndexLookup);
	EdgeElements->Get().Compact(OutRemappings.NewEdgeIndexLookup);
	TriangleElements->Get().Compact(OutRemappings.NewTriangleIndexLookup);
	PolygonElements->Get().Compact(OutRemappings.NewPolygonIndexLookup);
	PolygonGroupElements->Get().Compact(OutRemappings.NewPolygonGroupIndexLookup);

	FixUpElementIDs(OutRemappings);
}


void FMeshDescription::Remap(const FElementIDRemappings& Remappings)
{
	VertexElements->Get().Remap(Remappings.NewVertexIndexLookup);
	VertexInstanceElements->Get().Remap(Remappings.NewVertexInstanceIndexLookup);
	EdgeElements->Get().Remap(Remappings.NewEdgeIndexLookup);
	TriangleElements->Get().Remap(Remappings.NewTriangleIndexLookup);
	PolygonElements->Get().Remap(Remappings.NewPolygonIndexLookup);
	PolygonGroupElements->Get().Remap(Remappings.NewPolygonGroupIndexLookup);

	FixUpElementIDs(Remappings);
}


void FMeshDescription::FixUpElementIDs(const FElementIDRemappings& Remappings)
{
	// Fix up vertex index references in vertex instance array
	for (const FVertexInstanceID VertexInstanceID : VertexInstanceElements->Get().GetElementIDs())
	{
		VertexInstanceVertices[VertexInstanceID] = Remappings.GetRemappedVertexID(VertexInstanceVertices[VertexInstanceID]);
	}

	for (const FEdgeID EdgeID : EdgeElements->Get().GetElementIDs())
	{
		// Fix up vertex index references in Edges array
		for (int32 Index = 0; Index < 2; Index++)
		{
			EdgeVertices[EdgeID][Index] = Remappings.GetRemappedVertexID(EdgeVertices[EdgeID][Index]);
		}
	}

	for (const FTriangleID TriangleID : TriangleElements->Get().GetElementIDs())
	{
		// Fix up vertex instance references in Triangle
		for (int32 Index = 0; Index < 3; Index++)
		{
			TriangleVertexInstances[TriangleID][Index] = Remappings.GetRemappedVertexInstanceID(TriangleVertexInstances[TriangleID][Index]);
			TriangleEdges[TriangleID][Index] = Remappings.GetRemappedEdgeID(TriangleEdges[TriangleID][Index]);
			TriangleVertices[TriangleID][Index] = Remappings.GetRemappedVertexID(TriangleVertices[TriangleID][Index]);
		}

		TrianglePolygons[TriangleID] = Remappings.GetRemappedPolygonID(TrianglePolygons[TriangleID]);
		TrianglePolygonGroups[TriangleID] = Remappings.GetRemappedPolygonGroupID(TrianglePolygonGroups[TriangleID]);
	}

	for (const FPolygonID PolygonID : PolygonElements->Get().GetElementIDs())
	{
		PolygonPolygonGroups[PolygonID] = Remappings.GetRemappedPolygonGroupID(PolygonPolygonGroups[PolygonID]);
	}

	RebuildIndexers();
}


void FMeshDescription::CreateVertexInstance_Internal(const FVertexInstanceID VertexInstanceID, const FVertexID VertexID)
{
	VertexInstanceVertices[VertexInstanceID] = VertexID;
	VertexToVertexInstances.AddReferenceToKey(VertexID, VertexInstanceID);
}


template <template <typename...> class TContainer>
void FMeshDescription::DeleteVertexInstance_Internal(const FVertexInstanceID VertexInstanceID, TContainer<FVertexID>* InOutOrphanedVerticesPtr)
{
	checkSlow(VertexInstanceToTriangles.Find(VertexInstanceID).Num() == 0);

	const FVertexID VertexID = VertexInstanceVertices[VertexInstanceID];
	VertexToVertexInstances.RemoveReferenceFromKey(VertexID, VertexInstanceID);

	VertexInstanceElements->Get().Remove(VertexInstanceID);
	VertexInstanceToTriangles.RemoveKey(VertexInstanceID);

	// Always perform the Find() after the element has been completely deleted, so it won't be included in reindexing if the key is stale.
	// Note: removing the element above will clear the attributes to default, so remember the VertexID we're interested in.
	if (InOutOrphanedVerticesPtr &&
		VertexToVertexInstances.Find(VertexID).Num() == 0 &&
		VertexToEdges.Find(VertexID).Num() == 0)
	{
		AddUnique(*InOutOrphanedVerticesPtr, VertexID);
	}
}

void FMeshDescription::DeleteVertexInstance(const FVertexInstanceID VertexInstanceID, TArray<FVertexID>* InOutOrphanedVerticesPtr)
{
	DeleteVertexInstance_Internal<TArray>(VertexInstanceID, InOutOrphanedVerticesPtr);
}

void FMeshDescription::CreateEdge_Internal(const FEdgeID EdgeID, const FVertexID VertexID0, const FVertexID VertexID1)
{
	checkSlow(GetVertexPairEdge(VertexID0, VertexID1) == INDEX_NONE);

	TArrayView<FVertexID> EdgeVertexIDs = EdgeVertices[EdgeID];
	EdgeVertexIDs[0] = VertexID0;
	EdgeVertexIDs[1] = VertexID1;

	VertexToEdges.AddReferenceToKey(VertexID0, EdgeID);
	VertexToEdges.AddReferenceToKey(VertexID1, EdgeID);
}

template <template <typename...> class TContainer>
void FMeshDescription::DeleteEdge_Internal(const FEdgeID EdgeID, TContainer<FVertexID>* InOutOrphanedVerticesPtr)
{
	const FVertexID VertexID0 = EdgeVertices[EdgeID][0];
	const FVertexID VertexID1 = EdgeVertices[EdgeID][1];

	VertexToEdges.RemoveReferenceFromKey(VertexID0, EdgeID);
	VertexToEdges.RemoveReferenceFromKey(VertexID1, EdgeID);

	EdgeElements->Get().Remove(EdgeID);
	EdgeToTriangles.RemoveKey(EdgeID);

	// Always perform the Find() after the element has been completely deleted, so it won't be included in reindexing if the key is stale.
	// Note: removing the element above will clear the attributes to default, so remember the VertexID we're interested in.
	if (InOutOrphanedVerticesPtr)
	{
		if (VertexToEdges.Find(VertexID0).Num() == 0)
		{
			check(VertexToVertexInstances.Find(VertexID0).Num() == 0);
			AddUnique(*InOutOrphanedVerticesPtr, VertexID0);
		}

		if (VertexToEdges.Find(VertexID1).Num() == 0)
		{
			check(VertexToVertexInstances.Find(VertexID1).Num() == 0);
			AddUnique(*InOutOrphanedVerticesPtr, VertexID1);
		}
	}
}

void FMeshDescription::DeleteEdge(const FEdgeID EdgeID, TArray<FVertexID>* InOutOrphanedVerticesPtr)
{
	DeleteEdge_Internal<TArray>(EdgeID, InOutOrphanedVerticesPtr);
}

void FMeshDescription::CreateTriangle_Internal(const FTriangleID TriangleID, const FPolygonGroupID PolygonGroupID, TArrayView<const FVertexInstanceID> VertexInstanceIDs, TArray<FEdgeID>* OutEdgeIDs)
{
	if (OutEdgeIDs)
	{
		OutEdgeIDs->Empty();
	}

	// Fill out triangle vertex instances
	TArrayView<FVertexInstanceID> TriVertexInstanceIDs = TriangleVertexInstances[TriangleID];
	check(VertexInstanceIDs.Num() == 3);
	TriVertexInstanceIDs[0] = VertexInstanceIDs[0];
	TriVertexInstanceIDs[1] = VertexInstanceIDs[1];
	TriVertexInstanceIDs[2] = VertexInstanceIDs[2];

	// Fill out triangle polygon group
	TrianglePolygonGroups[TriangleID] = PolygonGroupID;
	PolygonGroupToTriangles.AddReferenceToKey(PolygonGroupID, TriangleID);

	// Make a polygon which will contain this triangle
	// @todo: make this optional; currently only exists for backwards compatibility
	FPolygonID PolygonID = PolygonElements->Get().Add();
	PolygonPolygonGroups[PolygonID] = PolygonGroupID;
	PolygonGroupToPolygons.AddReferenceToKey(PolygonGroupID, PolygonID);

	TrianglePolygons[TriangleID] = PolygonID;
	PolygonToTriangles.AddReferenceToKey(PolygonID, TriangleID);

	TArrayView<FVertexID> TriVertexIDs = TriangleVertices[TriangleID];
	TArrayView<FEdgeID> TriEdgeIDs = TriangleEdges[TriangleID];

	for (int32 Index = 0; Index < 3; ++Index)
	{
		const FVertexInstanceID VertexInstanceID = TriVertexInstanceIDs[Index];
		const FVertexInstanceID NextVertexInstanceID = TriVertexInstanceIDs[(Index == 2) ? 0 : Index + 1];

		const FVertexID ThisVertexID = VertexInstanceVertices[VertexInstanceID];
		const FVertexID NextVertexID = VertexInstanceVertices[NextVertexInstanceID];

		TriVertexIDs[Index] = ThisVertexID;
		// VertexToTriangles.AddReferenceToKey(ThisVertexID, TriangleID); // @todo: can add this, but need to consider how degenerates would work

		FEdgeID EdgeID = GetVertexPairEdge(ThisVertexID, NextVertexID);
		if (EdgeID == INDEX_NONE)
		{
			EdgeID = CreateEdge(ThisVertexID, NextVertexID);
			if (OutEdgeIDs)
			{
				OutEdgeIDs->Add(EdgeID);
			}
		}

		TriEdgeIDs[Index] = EdgeID;

		VertexInstanceToTriangles.AddReferenceToKey(VertexInstanceID, TriangleID);
		EdgeToTriangles.AddReferenceToKey(EdgeID, TriangleID);
	}
}


void FMeshDescription::SetTriangleUVIndices(const FTriangleID TriangleID, TArrayView<const FUVID> UVIDs, int32 UVChannel)
{
	TArrayView<FUVID> TriUVs = TriangleUVs.Get(TriangleID, UVChannel);
	TriUVs[0] = UVIDs[0];
	TriUVs[1] = UVIDs[1];
	TriUVs[2] = UVIDs[2];
	UVToTriangles.AddReferenceToKey(UVIDs[0], TriangleID, UVChannel);
	UVToTriangles.AddReferenceToKey(UVIDs[1], TriangleID, UVChannel);
	UVToTriangles.AddReferenceToKey(UVIDs[2], TriangleID, UVChannel);
}


template <template <typename...> class TContainer>
void FMeshDescription::DeleteTriangle_Internal(const FTriangleID TriangleID, TContainer<FEdgeID>* InOutOrphanedEdgesPtr, TContainer<FVertexInstanceID>* InOutOrphanedVertexInstancesPtr, TContainer<FPolygonGroupID>* InOutOrphanedPolygonGroupsPtr)
{
	const FPolygonGroupID PolygonGroupID = TrianglePolygonGroups[TriangleID];
	PolygonGroupToTriangles.RemoveReferenceFromKey(PolygonGroupID, TriangleID);

	const FPolygonID PolygonID = TrianglePolygons[TriangleID];
	PolygonToTriangles.RemoveReferenceFromKey(PolygonID, TriangleID);

	TArrayView<FVertexInstanceID> TriVertexInstanceIDs = TriangleVertexInstances[TriangleID];
	TArrayView<FEdgeID> TriEdgeIDs = TriangleEdges[TriangleID];

	for (int32 Index = 0; Index < 3; ++Index)
	{
		// Remove vertex instance from the triangle. Here's how we do that safely:
		// 1) Take a copy of the VertexInstance ID
		// 2) Remove reference from the VertexInstance to Triangles indexer
		// 3) Set the triangle VertexInstance to INDEX_NONE
		// 4) Ask the indexer if that VertexInstance ID has any triangle references, and if not, add it to the list.
		//
		// Step 3 is important because Find() below might need to reindex if the key is stale, and this would cause it to
		// re-add the triangle VertexInstance reference if it were still valid, since the triangle hasn't yet been deleted.
		//
		// We could also wait until the triangle were deleted before querying the indexer, but this would involve taking
		// copies of all the referenced element IDs first.

		const FVertexInstanceID VertexInstanceID = TriVertexInstanceIDs[Index];
			
		VertexInstanceToTriangles.RemoveReferenceFromKey(VertexInstanceID, TriangleID);
		TriVertexInstanceIDs[Index] = INDEX_NONE;
			
		if (InOutOrphanedVertexInstancesPtr && VertexInstanceToTriangles.Find(VertexInstanceID).Num() == 0)
		{
			AddUnique(*InOutOrphanedVertexInstancesPtr, VertexInstanceID);
		}

		// Remove edge from the triangle. Same rules as above apply!
			
		const FEdgeID EdgeID = TriEdgeIDs[Index];
			
		EdgeToTriangles.RemoveReferenceFromKey(EdgeID, TriangleID);
		TriEdgeIDs[Index] = INDEX_NONE;
			
		if (InOutOrphanedEdgesPtr && EdgeToTriangles.Find(EdgeID).Num() == 0)
		{
			AddUnique(*InOutOrphanedEdgesPtr, EdgeID);
		}

		// Remove UVs from the triangle if there are any

		for (int32 UVIndex = 0; UVIndex < TriangleUVs.GetNumChannels(); UVIndex++)
		{
			TArrayView<const FUVID> TriUVs = TriangleUVs.Get(TriangleID, UVIndex);
			FUVID UVID = TriUVs[Index];
			UVToTriangles.RemoveReferenceFromKey(UVID, TriangleID, UVIndex);
		}
	}

	if (PolygonToTriangles.Find(PolygonID).Num() == 0)
	{
		// If it was the only triangle in the polygon, delete the polygon too
		check(PolygonPolygonGroups[PolygonID] == PolygonGroupID);
		PolygonGroupToPolygons.RemoveReferenceFromKey(PolygonGroupID, PolygonID);

		PolygonElements->Get().Remove(PolygonID);
		PolygonToTriangles.RemoveKey(PolygonID);
	}

	TriangleElements->Get().Remove(TriangleID);

	// Check orphaned polygon groups after the triangle has been removed so that reindexing won't find it again.
	if (InOutOrphanedPolygonGroupsPtr && PolygonGroupToTriangles.Find(PolygonGroupID).Num() == 0)
	{
		AddUnique(*InOutOrphanedPolygonGroupsPtr, PolygonGroupID);
	}
}

void FMeshDescription::DeleteTriangle(const FTriangleID TriangleID, TArray<FEdgeID>* InOutOrphanedEdgesPtr, TArray<FVertexInstanceID>* InOutOrphanedVertexInstancesPtr, TArray<FPolygonGroupID>* InOutOrphanedPolygonGroupsPtr)
{
	DeleteTriangle_Internal<TArray>(TriangleID, InOutOrphanedEdgesPtr, InOutOrphanedVertexInstancesPtr, InOutOrphanedPolygonGroupsPtr);
}

void FMeshDescription::DeleteTriangles(const TArray<FTriangleID>& Triangles)
{
	TSet<FEdgeID> OrphanedEdges;
	TSet<FVertexInstanceID> OrphanedVertexInstances;
	TSet<FPolygonGroupID> OrphanedPolygonGroups;
	TSet<FVertexID> OrphanedVertices;

	for (FTriangleID TriangleID : Triangles)
	{
		DeleteTriangle_Internal<TSet>(TriangleID, &OrphanedEdges, &OrphanedVertexInstances, &OrphanedPolygonGroups);
	}
	for (FPolygonGroupID PolygonGroupID : OrphanedPolygonGroups)
	{
		DeletePolygonGroup(PolygonGroupID);
	}
	for (FVertexInstanceID VertexInstanceID : OrphanedVertexInstances)
	{
		DeleteVertexInstance_Internal<TSet>(VertexInstanceID, &OrphanedVertices);
	}
	for (FEdgeID EdgeID : OrphanedEdges)
	{
		DeleteEdge_Internal<TSet>(EdgeID, &OrphanedVertices);
	}
	for (FVertexID VertexID : OrphanedVertices)
	{
		DeleteVertex(VertexID);
	}
}

void FMeshDescription::CreatePolygon_Internal(const FPolygonID PolygonID, const FPolygonGroupID PolygonGroupID, TArrayView<const FVertexInstanceID> VertexInstanceIDs, TArray<FEdgeID>* OutEdgeIDs)
{
	if (OutEdgeIDs)
	{
		OutEdgeIDs->Empty();
	}

	check(PolygonGroupID != INDEX_NONE);
	PolygonPolygonGroups[PolygonID] = PolygonGroupID;
	PolygonGroupToPolygons.AddReferenceToKey(PolygonGroupID, PolygonID);

	const int32 NumVertices = VertexInstanceIDs.Num();
	for (int32 Index = 0; Index < NumVertices; ++Index)
	{
		const FVertexInstanceID ThisVertexInstanceID = VertexInstanceIDs[Index];
		const FVertexInstanceID NextVertexInstanceID = VertexInstanceIDs[(Index + 1 == NumVertices) ? 0 : Index + 1];
		const FVertexID ThisVertexID = VertexInstanceVertices[ThisVertexInstanceID];
		const FVertexID NextVertexID = VertexInstanceVertices[NextVertexInstanceID];

		// Create any missing perimeter edges here (not internal edges; they will be created when the polygon triangles are created)
		FEdgeID EdgeID = GetVertexPairEdge(ThisVertexID, NextVertexID);
		if (EdgeID == INDEX_NONE)
		{
			EdgeID = CreateEdge(ThisVertexID, NextVertexID);
			if (OutEdgeIDs)
			{
				OutEdgeIDs->Add(EdgeID);
			}
		}
	}

	CreatePolygonTriangles(PolygonID, VertexInstanceIDs);
}

template <template <typename...> class TContainer>
void FMeshDescription::DeletePolygon_Internal(const FPolygonID PolygonID, TContainer<FEdgeID>* InOutOrphanedEdgesPtr, TContainer<FVertexInstanceID>* InOutOrphanedVertexInstancesPtr, TContainer<FPolygonGroupID>* InOutOrphanedPolygonGroupsPtr)
{
	const FPolygonGroupID PolygonGroupID = PolygonPolygonGroups[PolygonID];

	// Remove constituent triangles
	TArrayView<const FTriangleID> TriangleIDs = PolygonToTriangles.Find<FTriangleID>(PolygonID);

	TArray<FEdgeID, TInlineAllocator<8>> InternalEdgesToRemove;
	InternalEdgesToRemove.Reserve(TriangleIDs.Num() - 1);

	// Disconnect edges and vertex instances
	for (const FTriangleID TriangleID : TriangleIDs)
	{
		TArrayView<FVertexInstanceID> TriVertexInstanceIDs = TriangleVertexInstances[TriangleID];
		TArrayView<FEdgeID> TriEdgeIDs = TriangleEdges[TriangleID];

		for (int32 Index = 0; Index < 3; ++Index)
		{
			const FEdgeID EdgeID = TriEdgeIDs[Index];
			if (IsEdgeInternal(EdgeID))
			{
				// Remove internal edges - but not yet.
				// For now just make a note of them.
				InternalEdgesToRemove.AddUnique(EdgeID);
			}
			else
			{
				EdgeToTriangles.RemoveReferenceFromKey(EdgeID, TriangleID);

				// Set the reference to INDEX_NONE here, so that it won't be picked up in the Find() below
				// if it needs to rebuild the key.
				TriEdgeIDs[Index] = INDEX_NONE;

				if (InOutOrphanedEdgesPtr && EdgeToTriangles.Find(EdgeID).Num() == 0)
				{
					AddUnique(*InOutOrphanedEdgesPtr, EdgeID);
				}
			}

			const FVertexInstanceID VertexInstanceID = TriVertexInstanceIDs[Index];
			VertexInstanceToTriangles.RemoveReferenceFromKey(VertexInstanceID, TriangleID);

			// Set the reference to INDEX_NONE here, so that it won't be picked up in the Find() below
			// if it needs to rebuild the key.
			TriVertexInstanceIDs[Index] = INDEX_NONE;

			if (InOutOrphanedVertexInstancesPtr && VertexInstanceToTriangles.Find(VertexInstanceID).Num() == 0)
			{
				AddUnique(*InOutOrphanedVertexInstancesPtr, VertexInstanceID);
			}
		}
	}

	// Remove triangles
	for (const FTriangleID TriangleID : TriangleIDs)
	{
		check(TrianglePolygonGroups[TriangleID] == PolygonGroupID);
		PolygonGroupToTriangles.RemoveReferenceFromKey(PolygonGroupID, TriangleID);
		TriangleElements->Get().Remove(TriangleID);
	}

	// Remove internal edges
	for (const FEdgeID EdgeID : InternalEdgesToRemove)
	{
		for (const FVertexID EdgeVertexID : EdgeVertices[EdgeID])
		{
			VertexToEdges.RemoveReferenceFromKey(EdgeVertexID, EdgeID);
		}

		EdgeElements->Get().Remove(EdgeID);
		EdgeToTriangles.RemoveKey(EdgeID);
	}

	PolygonGroupToPolygons.RemoveReferenceFromKey(PolygonGroupID, PolygonID);

	PolygonElements->Get().Remove(PolygonID);
	PolygonToTriangles.RemoveKey(PolygonID);

	if (InOutOrphanedPolygonGroupsPtr && PolygonGroupToPolygons.Find(PolygonGroupID).Num() == 0)
	{
		AddUnique(*InOutOrphanedPolygonGroupsPtr, PolygonGroupID);
	}
}

void FMeshDescription::DeletePolygon(const FPolygonID PolygonID, TArray<FEdgeID>* InOutOrphanedEdgesPtr, TArray<FVertexInstanceID>* InOutOrphanedVertexInstancesPtr, TArray<FPolygonGroupID>* InOutOrphanedPolygonGroupsPtr)
{
	DeletePolygon_Internal<TArray>(PolygonID, InOutOrphanedEdgesPtr, InOutOrphanedVertexInstancesPtr, InOutOrphanedPolygonGroupsPtr);
}

void FMeshDescription::DeletePolygons(const TArray<FPolygonID>& Polygons)
{
	TSet<FEdgeID> OrphanedEdges;
	TSet<FVertexInstanceID> OrphanedVertexInstances;
	TSet<FPolygonGroupID> OrphanedPolygonGroups;
	TSet<FVertexID> OrphanedVertices;

	for (FPolygonID PolygonID : Polygons)
	{
		DeletePolygon_Internal<TSet>(PolygonID, &OrphanedEdges, &OrphanedVertexInstances, &OrphanedPolygonGroups);
	}
	for (FPolygonGroupID PolygonGroupID : OrphanedPolygonGroups)
	{
		DeletePolygonGroup(PolygonGroupID);
	}
	for (FVertexInstanceID VertexInstanceID : OrphanedVertexInstances)
	{
		DeleteVertexInstance_Internal<TSet>(VertexInstanceID, &OrphanedVertices);
	}
	for (FEdgeID EdgeID : OrphanedEdges)
	{
		DeleteEdge_Internal<TSet>(EdgeID, &OrphanedVertices);
	}
	for (FVertexID VertexID : OrphanedVertices)
	{
		DeleteVertex(VertexID);
	}
}

bool FMeshDescription::IsVertexOrphaned(const FVertexID VertexID) const
{
	TArrayView<const FVertexInstanceID> VertexInstanceIDs = VertexToVertexInstances.Find<FVertexInstanceID>(VertexID);
	for (const FVertexInstanceID VertexInstanceID : VertexInstanceIDs)
	{
		TArrayView<const FTriangleID> TriangleIDs = VertexInstanceToTriangles.Find<FTriangleID>(VertexInstanceID);
		if (TriangleIDs.Num() > 0)
		{
			return false;
		}
	}

	return true;
}


FEdgeID FMeshDescription::GetVertexPairEdge(const FVertexID VertexID0, const FVertexID VertexID1) const
{
	TArrayView<const FEdgeID> ConnectedEdgeIDs = VertexToEdges.Find<FEdgeID>(VertexID0);
	for (const FEdgeID VertexConnectedEdgeID : ConnectedEdgeIDs)
	{
		/**
		 * VertexToEdges is rebuilt from on-disk EdgeVertices and can hand back a stale EdgeID on a
		 * corrupt mesh, leading to an out-of-bounds read in EdgeVertices[...] below. Treat as "no edge found".
		 */
		if (!EdgeElements->Get().IsValid(VertexConnectedEdgeID))
		{
			static bool bLoggedOnce = false;
			if (!bLoggedOnce)
			{
				bLoggedOnce = true;
				UE_LOGF(LogLoad, Warning, "FMeshDescription::GetVertexPairEdge skipped stale EdgeID %d from VertexToEdges indexer. Further occurrences will be silent.", VertexConnectedEdgeID.GetValue());
			}
			continue;
		}
		const FVertexID EdgeVertexID0 = EdgeVertices[VertexConnectedEdgeID][0];
		const FVertexID EdgeVertexID1 = EdgeVertices[VertexConnectedEdgeID][1];
		if ((EdgeVertexID0 == VertexID0 && EdgeVertexID1 == VertexID1) || (EdgeVertexID0 == VertexID1 && EdgeVertexID1 == VertexID0))
		{
			return VertexConnectedEdgeID;
		}
	}

	return INDEX_NONE;
}


FEdgeID FMeshDescription::GetVertexInstancePairEdge(const FVertexInstanceID VertexInstanceID0, const FVertexInstanceID VertexInstanceID1) const
{
	const FVertexID VertexID0 = VertexInstanceVertices[VertexInstanceID0];
	const FVertexID VertexID1 = VertexInstanceVertices[VertexInstanceID1];
	TArrayView<const FEdgeID> ConnectedEdgeIDs = VertexToEdges.Find<FEdgeID>(VertexID0);
	for (const FEdgeID VertexConnectedEdgeID : ConnectedEdgeIDs)
	{
		// See GetVertexPairEdge: same defensive guard against stale indexer entries.
		if (!EdgeElements->Get().IsValid(VertexConnectedEdgeID))
		{
			static bool bLoggedOnce = false;
			if (!bLoggedOnce)
			{
				bLoggedOnce = true;
				UE_LOGF(LogLoad, Warning, "FMeshDescription::GetVertexInstancePairEdge skipped stale EdgeID %d from VertexToEdges indexer. Further occurrences will be silent.", VertexConnectedEdgeID.GetValue());
			}
			continue;
		}
		const FVertexID EdgeVertexID0 = EdgeVertices[VertexConnectedEdgeID][0];
		const FVertexID EdgeVertexID1 = EdgeVertices[VertexConnectedEdgeID][1];
		if ((EdgeVertexID0 == VertexID0 && EdgeVertexID1 == VertexID1) || (EdgeVertexID0 == VertexID1 && EdgeVertexID1 == VertexID0))
		{
			return VertexConnectedEdgeID;
		}
	}

	return INDEX_NONE;
}


void FMeshDescription::SetPolygonVertexInstance(const FPolygonID PolygonID, const int32 PerimeterIndex, const FVertexInstanceID VertexInstanceID)
{
	TArrayView<const FTriangleID> Tris = PolygonToTriangles.Find<FTriangleID>(PolygonID);
	if (Tris.Num() == 1)
	{
		check(PerimeterIndex < 3);
		FVertexInstanceID OldVertexInstanceID = TriangleVertexInstances[Tris[0]][PerimeterIndex];
		check(VertexInstanceVertices[OldVertexInstanceID] == VertexInstanceVertices[VertexInstanceID]);
		VertexInstanceToTriangles.RemoveReferenceFromKey(OldVertexInstanceID, Tris[0]);
		TriangleVertexInstances[Tris[0]][PerimeterIndex] = VertexInstanceID;
		VertexInstanceToTriangles.AddReferenceToKey(VertexInstanceID, Tris[0]);
	}
	else
	{
		TArray<FVertexInstanceID, TInlineAllocator<8>> PolygonVertexInstances = GetPolygonVertexInstances<TInlineAllocator<8>>(PolygonID);
		check(PerimeterIndex < PolygonVertexInstances.Num());
		FVertexInstanceID OldVertexInstanceID = PolygonVertexInstances[PerimeterIndex];
		check(VertexInstanceVertices[OldVertexInstanceID] == VertexInstanceVertices[VertexInstanceID]);

		for (const FTriangleID Tri : Tris)
		{
			for (FVertexInstanceID& VertexInstance : TriangleVertexInstances[Tri])
			{
				if (VertexInstance == OldVertexInstanceID)
				{
					VertexInstanceToTriangles.RemoveReferenceFromKey(OldVertexInstanceID, Tri);
					VertexInstance = VertexInstanceID;
					VertexInstanceToTriangles.AddReferenceToKey(VertexInstanceID, Tri);
				}
			}
		}
	}
}


void FMeshDescription::SetPolygonVertexInstances(const FPolygonID PolygonID, TArrayView<const FVertexInstanceID> VertexInstanceIDs)
{
	RemovePolygonTriangles(PolygonID);
	CreatePolygonTriangles(PolygonID, VertexInstanceIDs);
}


FPlane FMeshDescription::ComputePolygonPlane(TArrayView<const FVertexInstanceID> PerimeterVertexInstanceIDs) const
{
	// NOTE: This polygon plane computation code is partially based on the implementation of "Newell's method" from Real-Time
	//       Collision Detection by Christer Ericson, published by Morgan Kaufmann Publishers, (c) 2005 Elsevier Inc

	// @todo mesheditor perf: For polygons that are just triangles, use a cross product to get the normal fast!
	// @todo mesheditor perf: We could skip computing the plane distance when we only need the normal
	// @todo mesheditor perf: We could cache these computed polygon normals; or just use the normal of the first three vertices' triangle if it is satisfactory in all cases
	// @todo mesheditor: For non-planar polygons, the result can vary. Ideally this should use the actual polygon triangulation as opposed to the arbitrary triangulation used here.

	FVector3f Centroid = FVector3f::ZeroVector;
	FVector Normal = FVector::ZeroVector;

	// Use 'Newell's Method' to compute a robust 'best fit' plane from the vertices of this polygon
	for (int32 VertexNumberI = PerimeterVertexInstanceIDs.Num() - 1, VertexNumberJ = 0; VertexNumberJ < PerimeterVertexInstanceIDs.Num(); VertexNumberI = VertexNumberJ, VertexNumberJ++)
	{
		const FVertexID VertexIDI = PerimeterVertexInstanceIDs[VertexNumberI];
		const FVector3f PositionI = VertexPositions[VertexInstanceVertices[VertexIDI]];

		const FVertexID VertexIDJ = PerimeterVertexInstanceIDs[VertexNumberJ];
		const FVector3f PositionJ = VertexPositions[VertexInstanceVertices[VertexIDJ]];

		Centroid += PositionJ;

		Normal.X += (PositionJ.Y - PositionI.Y) * (PositionI.Z + PositionJ.Z);
		Normal.Y += (PositionJ.Z - PositionI.Z) * (PositionI.X + PositionJ.X);
		Normal.Z += (PositionJ.X - PositionI.X) * (PositionI.Y + PositionJ.Y);
	}

	Normal.Normalize();

	// Construct a plane from the normal and centroid
	return FPlane(Normal, FVector::DotProduct(FVector(Centroid), Normal) / (float)PerimeterVertexInstanceIDs.Num());
}


FVector FMeshDescription::ComputePolygonNormal(TArrayView<const FVertexInstanceID> PerimeterVertexInstanceIDs) const
{
	// @todo mesheditor: Polygon normals are now computed and cached when changes are made to a polygon.
	// In theory, we can just return that cached value, but we need to check that there is nothing which relies on the value being correct before
	// the cache is updated at the end of a modification.
	const FPlane PolygonPlane = ComputePolygonPlane(PerimeterVertexInstanceIDs);
	const FVector PolygonNormal(PolygonPlane.X, PolygonPlane.Y, PolygonPlane.Z);
	return PolygonNormal;
}


/** Returns true if the triangle formed by the specified three positions has a normal that is facing the opposite direction of the reference normal */
static bool IsTriangleFlipped(const FVector ReferenceNormal, const FVector VertexPositionA, const FVector VertexPositionB, const FVector VertexPositionC)
{
	const FVector TriangleNormal = FVector::CrossProduct(
		VertexPositionC - VertexPositionA,
		VertexPositionB - VertexPositionA).GetSafeNormal();
	return (FVector::DotProduct(ReferenceNormal, TriangleNormal) <= 0.0f);
}


/** Given three direction vectors, indicates if A and B are on the same 'side' of Vec. */
static bool VectorsOnSameSide(const FVector& Vec, const FVector& A, const FVector& B, const float SameSideDotProductEpsilon)
{
	const FVector CrossA = FVector::CrossProduct(Vec, A);
	const FVector CrossB = FVector::CrossProduct(Vec, B);
	float DotWithEpsilon = SameSideDotProductEpsilon + FVector::DotProduct(CrossA, CrossB);
	return !(DotWithEpsilon < 0.0f);
}


/** Util to see if P lies within triangle created by A, B and C. */
static bool PointInTriangle(const FVector& A, const FVector& B, const FVector& C, const FVector& P, const float InsideTriangleDotProductEpsilon)
{
	// Cross product indicates which 'side' of the vector the point is on
	// If its on the same side as the remaining vert for all edges, then its inside.	
	return (VectorsOnSameSide(B - A, P - A, C - A, InsideTriangleDotProductEpsilon) &&
			VectorsOnSameSide(C - B, P - B, A - B, InsideTriangleDotProductEpsilon) &&
			VectorsOnSameSide(A - C, P - C, B - C, InsideTriangleDotProductEpsilon));
}


void FMeshDescription::FindPolygonPerimeter(const FPolygonID PolygonID, TArrayView<FEdgeID> Edges) const
{
	TArrayView<const FTriangleID> PolygonTris = PolygonToTriangles.Find<FTriangleID>(PolygonID);
	check(Edges.Num() == PolygonTris.Num() + 2);

	// Optimization for simple triangle
	if (PolygonTris.Num() == 1)
	{
		for (int I = 0; I < 3; I++)
		{
			Edges[I] = TriangleEdges[PolygonTris[0]][I];
		}
		return;
	}

	// Determine perimeter for arbitrary n-gon.
	// @todo: This process can undoubtedly be optimized

	// Build a set of all the perimeter edges
	TArray<FEdgeID, TInlineAllocator<8>> PerimeterEdges;
	TArray<int32, TInlineAllocator<8>> TriIndices;

	for (int32 TriIndex = 0; TriIndex < PolygonTris.Num(); TriIndex++)
	{
		FTriangleID PolygonTri = PolygonTris[TriIndex];

		TArrayView<const FEdgeID> TriEdges = TriangleEdges[PolygonTri];
		for (FEdgeID EdgeID : TriEdges)
		{
			int32 EdgeIndex = PerimeterEdges.Find(EdgeID);
			if (EdgeIndex != INDEX_NONE)
			{
				// If adding an edge which already exists, it must be an internal edge, so remove it again.
				PerimeterEdges.RemoveAtSwap(EdgeIndex, EAllowShrinking::No);
				TriIndices.RemoveAtSwap(EdgeIndex, EAllowShrinking::No);
			}
			else
			{
				PerimeterEdges.Add(EdgeID);
				TriIndices.Add(TriIndex);
			}
		}
	}
	check(PerimeterEdges.Num() == PolygonTris.Num() + 2);
	check(PerimeterEdges.Num() == TriIndices.Num());

	// Reorder edges to be adjacent by ensuring there's a mutual vertex in consecutive edges
	for (int32 EdgeIndex = 0; EdgeIndex < PerimeterEdges.Num() - 2; EdgeIndex++)
	{
		TArrayView<const FVertexID> EdgeVertexIDs = EdgeVertices[PerimeterEdges[EdgeIndex]];
		for (int32 NextEdgeIndex = EdgeIndex + 1; NextEdgeIndex < PerimeterEdges.Num(); NextEdgeIndex++)
		{
			TArrayView<const FVertexID> NextEdgeVertexIDs = EdgeVertices[PerimeterEdges[NextEdgeIndex]];
			if (EdgeVertexIDs[0] == NextEdgeVertexIDs[0] || EdgeVertexIDs[0] == NextEdgeVertexIDs[1] ||
				EdgeVertexIDs[1] == NextEdgeVertexIDs[0] || EdgeVertexIDs[1] == NextEdgeVertexIDs[1])
			{
				if (NextEdgeIndex > EdgeIndex + 1)
				{
					Swap(PerimeterEdges[EdgeIndex + 1], PerimeterEdges[NextEdgeIndex]);
					Swap(TriIndices[EdgeIndex + 1], TriIndices[NextEdgeIndex]);
				}
				break;
			}
		}
	}
	check(EdgeVertices[PerimeterEdges.Last()][0] == EdgeVertices[PerimeterEdges[0]][0] || EdgeVertices[PerimeterEdges.Last()][0] == EdgeVertices[PerimeterEdges[0]][1] ||
		  EdgeVertices[PerimeterEdges.Last()][1] == EdgeVertices[PerimeterEdges[0]][0] || EdgeVertices[PerimeterEdges.Last()][1] == EdgeVertices[PerimeterEdges[0]][1]);

	// Swap the winding order if incorrect
	FEdgeID FirstEdge = PerimeterEdges[0];
	FEdgeID SecondEdge = PerimeterEdges[1];

	// Get the triangle which the first edge lies in
	FTriangleID FirstTriangle = PolygonTris[TriIndices[0]];
	int32 TriEdgeIndex = TriangleEdges[FirstTriangle].Find(FirstEdge);

	// Get the end vertex of the edge as used in that triangle
	FVertexID SecondVertex = TriangleVertices[FirstTriangle][(TriEdgeIndex + 1) % 3];

	// If the second edge doesn't contain that end vertex, we need to reverse the order of the edges we just constructed
	if (EdgeVertices[SecondEdge][0] != SecondVertex && EdgeVertices[SecondEdge][1] != SecondVertex)
	{
		for (int32 I = 0; I < PerimeterEdges.Num() / 2; I++)
		{
			Swap(PerimeterEdges[I], PerimeterEdges[PerimeterEdges.Num() - 1 - I]);
			Swap(TriIndices[I], TriIndices[PerimeterEdges.Num() - 1 - I]);
		}
	}

	for (int I = 0; I < PerimeterEdges.Num(); I++)
	{
		Edges[I] = PerimeterEdges[I];
	}
}


void FMeshDescription::FindPolygonPerimeter(TArrayView<const FTriangleID> Triangles, TArrayView<TTuple<int32, int32>> Result) const
{
	// This constructs the perimeter indices for a polygon in the correct winding order from its constituent triangles
	check(Result.Num() == Triangles.Num() + 2);

	// Optimization for simple triangle
	if (Triangles.Num() == 1)
	{
		// Return triangle index 0; edges 0, 1, 2
		Result[0] = MakeTuple(0, 0);
		Result[1] = MakeTuple(0, 1);
		Result[2] = MakeTuple(0, 2);
		return;
	}

	// Determine perimeter for arbitrary n-gon.
	// @todo: This process can undoubtedly be optimized

	// Build a set of all the perimeter edges
	TArray<FEdgeID, TInlineAllocator<8>> PerimeterEdges;
	TArray<TTuple<int32, int32>, TInlineAllocator<8>> Indices;

	for (int32 TriIndex = 0; TriIndex < Triangles.Num(); TriIndex++)
	{
		FTriangleID PolygonTri = Triangles[TriIndex];

		TArrayView<const FEdgeID> TriEdges = TriangleEdges[PolygonTri];
		for (int32 EdgeIndex = 0; EdgeIndex < 3; EdgeIndex++)
		{
			FEdgeID EdgeID = TriEdges[EdgeIndex];
			int32 PerimeterIndex = PerimeterEdges.Find(EdgeID);
			if (PerimeterIndex != INDEX_NONE)
			{
				// If adding an edge which already exists, it must be an internal edge, so remove it again.
				PerimeterEdges.RemoveAtSwap(PerimeterIndex, EAllowShrinking::No);
				Indices.RemoveAtSwap(PerimeterIndex, EAllowShrinking::No);
			}
			else
			{
				PerimeterEdges.Add(EdgeID);
				Indices.Add(MakeTuple(TriIndex, EdgeIndex));
			}
		}
	}
	check(PerimeterEdges.Num() == Triangles.Num() + 2);
	check(PerimeterEdges.Num() == Indices.Num());

	// Reorder edges to be adjacent by ensuring there's a mutual vertex in consecutive edges
	for (int32 EdgeIndex = 0; EdgeIndex < PerimeterEdges.Num() - 2; EdgeIndex++)
	{
		TArrayView<const FVertexID> EdgeVertexIDs = EdgeVertices[PerimeterEdges[EdgeIndex]];
		for (int32 NextEdgeIndex = EdgeIndex + 1; NextEdgeIndex < PerimeterEdges.Num(); NextEdgeIndex++)
		{
			TArrayView<const FVertexID> NextEdgeVertexIDs = EdgeVertices[PerimeterEdges[NextEdgeIndex]];
			if (EdgeVertexIDs[0] == NextEdgeVertexIDs[0] || EdgeVertexIDs[0] == NextEdgeVertexIDs[1] ||
				EdgeVertexIDs[1] == NextEdgeVertexIDs[0] || EdgeVertexIDs[1] == NextEdgeVertexIDs[1])
			{
				if (NextEdgeIndex > EdgeIndex + 1)
				{
					Swap(PerimeterEdges[EdgeIndex + 1], PerimeterEdges[NextEdgeIndex]);
					Swap(Indices[EdgeIndex + 1], Indices[NextEdgeIndex]);
				}
				break;
			}
		}
	}
	check(EdgeVertices[PerimeterEdges.Last()][0] == EdgeVertices[PerimeterEdges[0]][0] || EdgeVertices[PerimeterEdges.Last()][0] == EdgeVertices[PerimeterEdges[0]][1] ||
		  EdgeVertices[PerimeterEdges.Last()][1] == EdgeVertices[PerimeterEdges[0]][0] || EdgeVertices[PerimeterEdges.Last()][1] == EdgeVertices[PerimeterEdges[0]][1]);

	// Swap the winding order if incorrect
	FEdgeID FirstEdge = PerimeterEdges[0];
	FEdgeID SecondEdge = PerimeterEdges[1];

	// Get the triangle which the first edge lies in
	FTriangleID FirstTriangle = Triangles[Indices[0].Get<0>()];
	int32 TriEdgeIndex = TriangleEdges[FirstTriangle].Find(FirstEdge);

	// Get the end vertex of the edge as used in that triangle
	FVertexID SecondVertex = TriangleVertices[FirstTriangle][(TriEdgeIndex + 1) % 3];

	// If the second edge doesn't contain that end vertex, we need to reverse the order of the edges we just constructed
	if (EdgeVertices[SecondEdge][0] != SecondVertex && EdgeVertices[SecondEdge][1] != SecondVertex)
	{
		for (int32 I = 0; I < Indices.Num() / 2; I++)
		{
			Swap(Indices[I], Indices[PerimeterEdges.Num() - 1 - I]);
		}
	}

	for (int I = 0; I < Indices.Num(); I++)
	{
		Result[I] = Indices[I];
	}
}


void FMeshDescription::ComputePolygonTriangulation(const FPolygonID PolygonID)
{
	TArrayView<const FTriangleID> TriangleIDs = PolygonToTriangles.Find<FTriangleID>(PolygonID);

	// Not valid to call this on an untriangulated polygon - an untriangulated polygon is no longer valid at all.
	check(TriangleIDs.Num() > 0);

	// If polygon was already triangulated, and only has three vertices, no need to do anything here
	if (TriangleIDs.Num() == 1)
	{
		return;
	}

	// Get the current perimeter vertex instances
	TArray<FVertexInstanceID, TInlineAllocator<8>> PolygonVertexInstanceIDs = GetPolygonVertexInstances<TInlineAllocator<8>>(PolygonID);

	// Remove existing triangles
	RemovePolygonTriangles(PolygonID);

	// Perform the triangulation
	CreatePolygonTriangles(PolygonID, PolygonVertexInstanceIDs);
}


void FMeshDescription::RemovePolygonTriangles(const FPolygonID PolygonID)
{
	TArrayView<const FTriangleID> TriangleIDs = PolygonToTriangles.Find<FTriangleID>(PolygonID);
	const FPolygonGroupID PolygonGroupID = PolygonPolygonGroups[PolygonID];

	// Remove currently configured triangles
	TArray<FEdgeID> InternalEdgesToRemove;
	InternalEdgesToRemove.Reserve(TriangleIDs.Num() - 1);

	for (const FTriangleID TriangleID : TriangleIDs)
	{
		TArrayView<FVertexInstanceID> TriVertexInstanceIDs = TriangleVertexInstances[TriangleID];
		TArrayView<FEdgeID> TriEdgeIDs = TriangleEdges[TriangleID];

		for (int32 Index = 0; Index < 3; ++Index)
		{
			VertexInstanceToTriangles.RemoveReferenceFromKey(TriVertexInstanceIDs[Index], TriangleID);

			const FEdgeID EdgeID = TriEdgeIDs[Index];
			if (IsEdgeInternal(EdgeID))
			{
				InternalEdgesToRemove.Add(EdgeID);
			}

			EdgeToTriangles.RemoveReferenceFromKey(EdgeID, TriangleID);
		}

		PolygonGroupToTriangles.RemoveReferenceFromKey(PolygonGroupID, TriangleID);
		TriangleElements->Get().Remove(TriangleID);
	}

	PolygonToTriangles.RemoveKey(PolygonID);

	// Remove internal edges
	for (const FEdgeID EdgeID : InternalEdgesToRemove)
	{
		for (const FVertexID EdgeVertexID : EdgeVertices[EdgeID])
		{
			VertexToEdges.RemoveReferenceFromKey(EdgeVertexID, EdgeID);
		}

		EdgeElements->Get().Remove(EdgeID);
		EdgeToTriangles.RemoveKey(EdgeID);
	}
}


void FMeshDescription::SplitPolygon(FPolygonID OriginalPolygonID)
{
	TArrayView<const FTriangleID> TriangleIDs = PolygonToTriangles.Find<FTriangleID>(OriginalPolygonID);
	if (TriangleIDs.Num() == 1)
	{
		return;
	}

	const FPolygonGroupID PolygonGroupID = PolygonPolygonGroups[OriginalPolygonID];
	TArray<FTriangleID, TInlineAllocator<64>> TriangleIDsCopy(TriangleIDs);

	for (int32 TriangleIndex = 1; TriangleIndex < TriangleIDsCopy.Num(); TriangleIndex++)
	{
		FPolygonID NewPolygonID = PolygonElements->Get().Add();
		PolygonPolygonGroups[NewPolygonID] = PolygonGroupID;
		PolygonGroupToPolygons.AddReferenceToKey(PolygonGroupID, NewPolygonID);

		FTriangleID TriangleID = TriangleIDsCopy[TriangleIndex];
		PolygonToTriangles.RemoveReferenceFromKey(OriginalPolygonID, TriangleID);
		TrianglePolygons[TriangleID] = NewPolygonID;
		PolygonToTriangles.AddReferenceToKey(NewPolygonID, TriangleID);
	}
}


void FMeshDescription::CreatePolygonTriangles(const FPolygonID PolygonID, TArrayView<const FVertexInstanceID> VertexInstanceIDs)
{
	FPolygonGroupID PolygonGroupID = PolygonPolygonGroups[PolygonID];

	// If perimeter only has 3 vertices, just add a single triangle and return
	if (VertexInstanceIDs.Num() == 3)
	{
		const FTriangleID TriangleID = TriangleElements->Get().Add();

		// Fill out triangle vertex instances
		TArrayView<FVertexInstanceID> TriVertexInstanceIDs = TriangleVertexInstances[TriangleID];
		TriVertexInstanceIDs[0] = VertexInstanceIDs[0];
		TriVertexInstanceIDs[1] = VertexInstanceIDs[1];
		TriVertexInstanceIDs[2] = VertexInstanceIDs[2];

		// Fill out triangle polygon group
		TrianglePolygonGroups[TriangleID] = PolygonGroupID;
		PolygonGroupToTriangles.AddReferenceToKey(PolygonGroupID, TriangleID);

		// Fill out triangle polygon
		TrianglePolygons[TriangleID] = PolygonID;
		PolygonToTriangles.AddReferenceToKey(PolygonID, TriangleID);

		TArrayView<FVertexID> TriVertexIDs = TriangleVertices[TriangleID];
		TArrayView<FEdgeID> TriEdgeIDs = TriangleEdges[TriangleID];

		for (int32 Index = 0; Index < 3; ++Index)
		{
			const FVertexInstanceID VertexInstanceID = TriVertexInstanceIDs[Index];
			const FVertexInstanceID NextVertexInstanceID = TriVertexInstanceIDs[(Index == 2) ? 0 : Index + 1];

			const FVertexID ThisVertexID = VertexInstanceVertices[VertexInstanceID];
			const FVertexID NextVertexID = VertexInstanceVertices[NextVertexInstanceID];

			TriVertexIDs[Index] = ThisVertexID;

			FEdgeID EdgeID = GetVertexPairEdge(ThisVertexID, NextVertexID);
			check(EdgeID != INDEX_NONE);
			TriEdgeIDs[Index] = EdgeID;

			VertexInstanceToTriangles.AddReferenceToKey(VertexInstanceID, TriangleID);
			EdgeToTriangles.AddReferenceToKey(EdgeID, TriangleID);
		}

		return;
	}

	// NOTE: This polygon triangulation code is partially based on the ear cutting algorithm described on
	//       page 497 of the book "Real-time Collision Detection", published in 2005.

	// @todo mesheditor: Perhaps should always attempt to triangulate by splitting polygons along the shortest edge, for better determinism.

	// First figure out the polygon normal.  We need this to determine which triangles are convex, so that
	// we can figure out which ears to clip
	const FVector PolygonNormal = ComputePolygonNormal(VertexInstanceIDs);

	// Make a simple linked list array of the previous and next vertex numbers, for each vertex number
	// in the polygon.  This will just save us having to iterate later on.
	TArray<int32> PrevVertexNumbers;
	TArray<int32> NextVertexNumbers;
	TArray<FVector3f> PolyVertexPositions;
	int32 PolygonVertexCount = VertexInstanceIDs.Num();
	{
		PrevVertexNumbers.SetNumUninitialized(PolygonVertexCount, EAllowShrinking::No);
		NextVertexNumbers.SetNumUninitialized(PolygonVertexCount, EAllowShrinking::No);
		PolyVertexPositions.SetNumUninitialized(PolygonVertexCount, EAllowShrinking::No);

		for (int32 VertexNumber = 0; VertexNumber < PolygonVertexCount; ++VertexNumber)
		{
			PrevVertexNumbers[VertexNumber] = VertexNumber - 1;
			NextVertexNumbers[VertexNumber] = VertexNumber + 1;

			PolyVertexPositions[VertexNumber] = VertexPositions[GetVertexInstanceVertex(VertexInstanceIDs[VertexNumber])];
		}
		PrevVertexNumbers[0] = PolygonVertexCount - 1;
		NextVertexNumbers[PolygonVertexCount - 1] = 0;
	}

	int32 EarVertexNumber = 0;
	int32 EarTestCount = 0;
	for (int32 RemainingVertexCount = PolygonVertexCount; RemainingVertexCount >= 3; )
	{
		bool bIsEar = true;

		// If we're down to only a triangle, just treat it as an ear.  Also, if we've tried every possible candidate
		// vertex looking for an ear, go ahead and just treat the current vertex as an ear.  This can happen when 
		// vertices are colinear or other degenerate cases.
		if (RemainingVertexCount > 3 && EarTestCount < RemainingVertexCount)
		{
			const FVector PrevVertexPosition(PolyVertexPositions[PrevVertexNumbers[EarVertexNumber]]);
			const FVector EarVertexPosition(PolyVertexPositions[EarVertexNumber]);
			const FVector NextVertexPosition(PolyVertexPositions[NextVertexNumbers[EarVertexNumber]]);

			// Figure out whether the potential ear triangle is facing the same direction as the polygon
			// itself.  If it's facing the opposite direction, then we're dealing with a concave triangle
			// and we'll skip it for now.
			if (!IsTriangleFlipped(PolygonNormal, PrevVertexPosition, EarVertexPosition, NextVertexPosition))
			{
				int32 TestVertexNumber = NextVertexNumbers[NextVertexNumbers[EarVertexNumber]];

				do
				{
					// Test every other remaining vertex to make sure that it doesn't lie inside our potential ear
					// triangle.  If we find a vertex that's inside the triangle, then it cannot actually be an ear.
					const FVector TestVertexPosition(PolyVertexPositions[TestVertexNumber]);
					if (PointInTriangle(PrevVertexPosition, EarVertexPosition, NextVertexPosition, TestVertexPosition, SMALL_NUMBER))
					{
						bIsEar = false;
						break;
					}

					TestVertexNumber = NextVertexNumbers[TestVertexNumber];

				} while (TestVertexNumber != PrevVertexNumbers[EarVertexNumber]);
			}
			else
			{
				bIsEar = false;
			}
		}

		if (bIsEar)
		{
			// OK, we found an ear!  Let's save this triangle in our output buffer.
			// This will also create any missing internal edges.
			{
				// Add a new triangle
				const FTriangleID TriangleID = TriangleElements->Get().Add();

				// Fill out triangle vertex instances
				TArrayView<FVertexInstanceID> TriVertexInstanceIDs = TriangleVertexInstances[TriangleID];
				TriVertexInstanceIDs[0] = VertexInstanceIDs[PrevVertexNumbers[EarVertexNumber]];
				TriVertexInstanceIDs[1] = VertexInstanceIDs[EarVertexNumber];
				TriVertexInstanceIDs[2] = VertexInstanceIDs[NextVertexNumbers[EarVertexNumber]];

				// Fill out triangle polygon group
				TrianglePolygonGroups[TriangleID] = PolygonGroupID;
				PolygonGroupToTriangles.AddReferenceToKey(PolygonGroupID, TriangleID);

				// Fill out triangle polygon
				TrianglePolygons[TriangleID] = PolygonID;
				PolygonToTriangles.AddReferenceToKey(PolygonID, TriangleID);

				TArrayView<FVertexID> TriVertexIDs = TriangleVertices[TriangleID];
				TArrayView<FEdgeID> TriEdgeIDs = TriangleEdges[TriangleID];

				for (int32 Index = 0; Index < 3; ++Index)
				{
					const FVertexInstanceID VertexInstanceID = TriVertexInstanceIDs[Index];
					const FVertexInstanceID NextVertexInstanceID = TriVertexInstanceIDs[(Index == 2) ? 0 : Index + 1];

					const FVertexID ThisVertexID = VertexInstanceVertices[VertexInstanceID];
					const FVertexID NextVertexID = VertexInstanceVertices[NextVertexInstanceID];

					TriVertexIDs[Index] = ThisVertexID;

					FEdgeID EdgeID = GetVertexPairEdge(ThisVertexID, NextVertexID);
					if (EdgeID == INDEX_NONE)
					{
						// This must be an internal edge (as perimeter edges will already be defined)
						EdgeID = CreateEdge(ThisVertexID, NextVertexID);
					}
					TriEdgeIDs[Index] = EdgeID;

					VertexInstanceToTriangles.AddReferenceToKey(VertexInstanceID, TriangleID);
					EdgeToTriangles.AddReferenceToKey(EdgeID, TriangleID);
				}
			}

			// Update our linked list.  We're effectively cutting off the ear by pointing the ear vertex's neighbors to
			// point at their next sequential neighbor, and reducing the remaining vertex count by one.
			{
				NextVertexNumbers[PrevVertexNumbers[EarVertexNumber]] = NextVertexNumbers[EarVertexNumber];
				PrevVertexNumbers[NextVertexNumbers[EarVertexNumber]] = PrevVertexNumbers[EarVertexNumber];
				--RemainingVertexCount;
			}

			// Move on to the previous vertex in the list, now that this vertex was cut
			EarVertexNumber = PrevVertexNumbers[EarVertexNumber];

			EarTestCount = 0;
		}
		else
		{
			// The vertex is not the ear vertex, because it formed a triangle that either had a normal which pointed in the opposite direction
			// of the polygon, or at least one of the other polygon vertices was found to be inside the triangle.  Move on to the next vertex.
			EarVertexNumber = NextVertexNumbers[EarVertexNumber];

			// Keep track of how many ear vertices we've tested, so that if we exhaust all remaining vertices, we can
			// fall back to clipping the triangle and adding it to our mesh anyway.  This is important for degenerate cases.
			++EarTestCount;
		}
	}
}


FBoxSphereBounds FMeshDescription::GetBounds() const
{
	FBoxSphereBounds BoundingBoxAndSphere;

	FBox BoundingBox;
	BoundingBox.Init();

	for (const FVertexID VertexID : Vertices().GetElementIDs())
	{
		if (!IsVertexOrphaned(VertexID))
		{
			BoundingBox += FVector(VertexPositions[VertexID]);
		}
	}

	BoundingBox.GetCenterAndExtents(BoundingBoxAndSphere.Origin, BoundingBoxAndSphere.BoxExtent);

	// Calculate the bounding sphere, using the center of the bounding box as the origin.
	BoundingBoxAndSphere.SphereRadius = 0.0f;

	for (const FVertexID VertexID : Vertices().GetElementIDs())
	{
		if (!IsVertexOrphaned(VertexID))
		{
			BoundingBoxAndSphere.SphereRadius = FMath::Max<FVector::FReal>((FVector(VertexPositions[VertexID]) - BoundingBoxAndSphere.Origin).Size(), BoundingBoxAndSphere.SphereRadius);
		}
	}

	return BoundingBoxAndSphere;
}

void FMeshDescription::TriangulateMesh()
{
	// Perform triangulation directly into mesh polygons
	for( const FPolygonID PolygonID : Polygons().GetElementIDs() )
	{
		ComputePolygonTriangulation( PolygonID );
	}
}


void FMeshDescription::SetNumUVChannels(const int32 NumUVChannels)
{
	UVElements->SetNumChannels(NumUVChannels);
	TriangleUVs = TriangleElements->Get().GetAttributes().RegisterIndexAttribute<FUVID[3]>(MeshAttribute::Triangle::UVIndex, NumUVChannels);

	// Ensure every UV element channel has a UVCoordinate attribute
	for (int32 Index = 0; Index < NumUVChannels; Index++)
	{
		UVElements->Get(Index).GetAttributes().RegisterAttribute(MeshAttribute::UV::UVCoordinate, 1, FVector2f::ZeroVector, EMeshAttributeFlags::Lerpable);
	}
}


namespace MeshAttribute_
{
	namespace Vertex
	{
		const FName CornerSharpness("CornerSharpness");
	}

	namespace VertexInstance
	{
		const FName TextureCoordinate("TextureCoordinate");
		const FName Normal("Normal");
		const FName Tangent("Tangent");
		const FName BinormalSign("BinormalSign");
		const FName Color("Color");
	}

	namespace Edge
	{
		const FName IsHard("IsHard");
		const FName IsUVSeam("IsUVSeam");
		const FName CreaseSharpness("CreaseSharpness");
	}

	namespace Polygon
	{
		const FName Normal("Normal");
		const FName Tangent("Tangent");
		const FName Binormal("Binormal");
		const FName Center("Center");
	}

	namespace PolygonGroup
	{
		const FName ImportedMaterialSlotName("ImportedMaterialSlotName");
		const FName EnableCollision("EnableCollision");
		const FName CastShadow("CastShadow");
	}
}


float FMeshDescription::GetPolygonCornerAngleForVertex(const FPolygonID PolygonID, const FVertexID VertexID) const
{
	TArray<FVertexInstanceID> PolygonVertexInstanceIDs = GetPolygonVertexInstances(PolygonID);

	// Lambda function which returns the inner angle at a given index on a polygon contour
	auto GetContourAngle = [this](const TArray<FVertexInstanceID>& VertexInstanceIDs, const int32 ContourIndex)
	{
		const int32 NumVertices = VertexInstanceIDs.Num();

		const int32 PrevIndex = (ContourIndex + NumVertices - 1) % NumVertices;
		const int32 NextIndex = (ContourIndex + 1) % NumVertices;

		const FVertexID PrevVertexID = GetVertexInstanceVertex(VertexInstanceIDs[PrevIndex]);
		const FVertexID ThisVertexID = GetVertexInstanceVertex(VertexInstanceIDs[ContourIndex]);
		const FVertexID NextVertexID = GetVertexInstanceVertex(VertexInstanceIDs[NextIndex]);

		const FVector PrevVertexPosition(VertexPositions[PrevVertexID]);
		const FVector ThisVertexPosition(VertexPositions[ThisVertexID]);
		const FVector NextVertexPosition(VertexPositions[NextVertexID]);

		const FVector Direction1 = (PrevVertexPosition - ThisVertexPosition).GetSafeNormal();
		const FVector Direction2 = (NextVertexPosition - ThisVertexPosition).GetSafeNormal();

		return FMath::Acos(FVector::DotProduct(Direction1, Direction2));
	};

	auto IsVertexInstancedFromThisVertex = [this, VertexID](const FVertexInstanceID VertexInstanceID)
	{
		return this->GetVertexInstanceVertex(VertexInstanceID) == VertexID;
	};

	// First look for the vertex instance in the perimeter
	int32 ContourIndex = PolygonVertexInstanceIDs.IndexOfByPredicate(IsVertexInstancedFromThisVertex);
	if (ContourIndex != INDEX_NONE)
	{
		// Return the internal angle if found
		return GetContourAngle(PolygonVertexInstanceIDs, ContourIndex);
	}

	// Found nothing; return 0
	return 0.0f;
}

FBox FMeshDescription::ComputeBoundingBox() const
{
	FBox BoundingBox(ForceInit);

	for (const FVertexID VertexID : Vertices().GetElementIDs())
	{
		BoundingBox += FVector(VertexPositions[VertexID]);
	}

	return BoundingBox;
}


void FMeshDescription::ReverseTriangleFacing(const FTriangleID TriangleID)
{
	TArrayView<FVertexInstanceID> TriVertexInstances = TriangleVertexInstances[TriangleID];
	TArrayView<FVertexID> TriVertices = TriangleVertices[TriangleID];
	TArrayView<FEdgeID> TriEdges = TriangleEdges[TriangleID];
	Swap(TriVertexInstances[0], TriVertexInstances[1]);
	Swap(TriVertices[0], TriVertices[1]);
	Swap(TriEdges[1], TriEdges[2]);
}


void FMeshDescription::ReversePolygonFacing(const FPolygonID PolygonID)
{
	// Build a reverse perimeter
	TArray<FVertexInstanceID> Contour = GetPolygonVertexInstances(PolygonID);
	for (int32 i = 0; i < Contour.Num() / 2; ++i)
	{
		Swap(Contour[i], Contour[Contour.Num() - i - 1]);
	}

	RemovePolygonTriangles(PolygonID);
	CreatePolygonTriangles(PolygonID, Contour);
}


void FMeshDescription::ReverseAllPolygonFacing()
{
	// Perform triangulation directly into mesh polygons
	for (const FPolygonID PolygonID : Polygons().GetElementIDs())
	{
		ReversePolygonFacing(PolygonID);
	}
}


void FMeshDescription::RemapPolygonGroups(const TMap<FPolygonGroupID, FPolygonGroupID>& Remap)
{
	TPolygonGroupAttributesRef<FName> PolygonGroupNames = PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute_::PolygonGroup::ImportedMaterialSlotName);

	struct FOldPolygonGroupData
	{
		FName Name;
		TArray<FTriangleID> Triangles;
		TArray<FPolygonID> Polygons;
	};

	TMap<FPolygonGroupID, FOldPolygonGroupData> OldData;
	for (const FPolygonGroupID PolygonGroupID : PolygonGroups().GetElementIDs())
	{
		if (!Remap.Contains(PolygonGroupID) || PolygonGroupID == Remap[PolygonGroupID])
		{
			//No need to change this one
			continue;
		}
		FOldPolygonGroupData& PolygonGroupData = OldData.FindOrAdd(PolygonGroupID);
		PolygonGroupData.Name = PolygonGroupNames[PolygonGroupID];
		PolygonGroupData.Triangles = PolygonGroupToTriangles.Find<FTriangleID>(PolygonGroupID);
		PolygonGroupData.Polygons = PolygonGroupToPolygons.Find<FPolygonID>(PolygonGroupID);
		PolygonGroupElements->Get().Remove(PolygonGroupID);
		PolygonGroupToPolygons.RemoveKey(PolygonGroupID);
		PolygonGroupToTriangles.RemoveKey(PolygonGroupID);
	}

	for (auto Kvp : OldData)
	{
		FPolygonGroupID GroupID = Kvp.Key;
		FPolygonGroupID ToGroupID = Remap[GroupID];
		if (!PolygonGroups().IsValid(ToGroupID))
		{
			CreatePolygonGroupWithID(ToGroupID);
		}
		PolygonGroupNames[ToGroupID] = Kvp.Value.Name;

		for (const FTriangleID TriangleID : Kvp.Value.Triangles)
		{
			TrianglePolygonGroups[TriangleID] = ToGroupID;
			PolygonGroupToTriangles.AddReferenceToKey(ToGroupID, TriangleID);
		}

		for (const FPolygonID PolygonID : Kvp.Value.Polygons)
		{
			PolygonPolygonGroups[PolygonID] = ToGroupID;
			PolygonGroupToPolygons.AddReferenceToKey(ToGroupID, PolygonID);
		}
	}
}

void FMeshDescription::TransferPolygonGroup(FPolygonGroupID SourceID, FPolygonGroupID DestinationID)
{
	if (SourceID == DestinationID || !IsPolygonGroupValid(SourceID) || !IsPolygonGroupValid(DestinationID))
	{
		//Cannot transfer on self or if we have invalid polygon group ID
		return;
	}
	TArray<FTriangleID> Triangles;
	Triangles = PolygonGroupToTriangles.Find<FTriangleID>(SourceID);
	TArray<FPolygonID> Polygons;
	Polygons = PolygonGroupToPolygons.Find<FPolygonID>(SourceID);
	PolygonGroupElements->Get().Remove(SourceID);
	PolygonGroupToPolygons.RemoveKey(SourceID);
	PolygonGroupToTriangles.RemoveKey(SourceID);

	for (const FTriangleID TriangleID : Triangles)
	{
		TrianglePolygonGroups[TriangleID] = DestinationID;
		PolygonGroupToTriangles.AddReferenceToKey(DestinationID, TriangleID);
	}

	for (const FPolygonID PolygonID : Polygons)
	{
		PolygonPolygonGroups[PolygonID] = DestinationID;
		PolygonGroupToPolygons.AddReferenceToKey(DestinationID, PolygonID);
	}
}



#if WITH_EDITORONLY_DATA

TConstArrayView<FGuid> FMeshDescriptionBulkData::GetMeshDescriptionCustomVersions()
{
	// Wrap the aggregation in a static struct to ensure that the compiler only initializes it once and
	// in a thread-safe manner.
	static struct FVersionAggregator
	{
		FVersionAggregator()
		{
			for (const TPair<FGuid, int32>& Version: FMeshDescription::GetSerializationFormatVersions())
			{
				Versions.Add(Version.Key);
			}
		}
		
		TArray<FGuid> Versions;
	} AggregatedVersions;
	
	return AggregatedVersions.Versions;
}


void FMeshDescriptionBulkData::Serialize( FArchive& Ar, UObject* Owner )
{
	for (const FGuid& VersionGuid: GetMeshDescriptionCustomVersions())
	{
		Ar.UsingCustomVersion(VersionGuid);
	}

	// Make sure to serialize only actual data
	if (Ar.ShouldSkipBulkData() || Ar.IsObjectReferenceCollector())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshDescriptionBulkData::Serialize);

	if ( Ar.IsTransacting() )
	{
		// If transacting, keep these members alive the other side of an undo, otherwise their values will get lost
		Ar << UEVersion;
		Ar << LicenseeUEVersion;
		CustomVersions.Serialize( Ar );
		Ar << bBulkDataUpdated;
	}
	else
	{
		if ( Ar.IsSaving() )
		{
			// If the bulk data hasn't been updated since this was loaded, there's a possibility that it has old versioning.
			// Explicitly load and resave the FMeshDescription so that its version is in sync with the FMeshDescriptionBulkData.
			UpdateMeshDescriptionFormat();
		}
	}

	bool bSerializedOldDataTypes = false;
	FByteBulkData TempBulkData;
	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::VirtualizedBulkDataHaveUniqueGuids)
	{
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::MeshDescriptionVirtualization)
		{
			// Serialize the old BulkData format and mark that we require a conversion after the guid has been serialized
			TempBulkData.Serialize(Ar, Owner);
			bSerializedOldDataTypes = true;
		}
		else
		{
			BulkData.Serialize(Ar, Owner, false /* bAllowRegister */);
			BulkData.CreateLegacyUniqueIdentifier(Owner);
		}
	}
	else
	{
		BulkData.Serialize(Ar, Owner);
	}

	if( Ar.IsLoading() && Ar.CustomVer( FEditorObjectVersion::GUID ) < FEditorObjectVersion::MeshDescriptionBulkDataGuid )
	{
		FPlatformMisc::CreateGuid( Guid );
	}
	else
	{
		Ar << Guid;
	}
	
	// MeshDescriptionBulkData contains a bGuidIsHash so we can benefit from DDC caching.
	if( Ar.IsLoading() && Ar.CustomVer( FEnterpriseObjectVersion::GUID ) < FEnterpriseObjectVersion::MeshDescriptionBulkDataGuidIsHash )
	{
		bGuidIsHash = false;
	}
	else
	{
		Ar << bGuidIsHash;
	}

	if (!Ar.IsTransacting() && Ar.IsLoading())
	{
		// If loading, take a copy of the package's version information, so it can be applied when unpacking
		// MeshDescription from the bulk data.
		BulkData.GetBulkDataVersions(Ar, UEVersion, LicenseeUEVersion, CustomVersions);

	}

	// Needs to be after the guid is serialized
	if (bSerializedOldDataTypes)
	{
		BulkData.CreateFromBulkData(TempBulkData, Guid, Owner);
	}
}


void FMeshDescriptionBulkData::SaveMeshDescription( FMeshDescription& MeshDescription )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshDescriptionBulkData::SaveMeshDescription);

#if WITH_EDITOR
	FRWScopeLock ScopeLock(BulkDataLock, SLT_Write);
#endif

	BulkData.Reset();

	if( !MeshDescription.IsEmpty() )
	{
		const bool bIsPersistent = true;
		UE::Serialization::FEditorBulkDataWriter Ar(BulkData, bIsPersistent);
		Ar << MeshDescription;

		// Preserve versions at save time so we can reuse the same ones when reloading direct from memory
		UEVersion = Ar.UEVer();
		LicenseeUEVersion = Ar.LicenseeUEVer();
		CustomVersions = Ar.GetCustomVersions();
	}

	if (bGuidIsHash)
	{
		UseHashAsGuid();
	}
	else
	{
		FPlatformMisc::CreateGuid( Guid );
	}

	// Mark the MeshDescriptionBulkData as having been updated.
	// This means we know that its version is up-to-date.
	bBulkDataUpdated = true;
}

void FMeshDescriptionBulkData::LoadMeshDescription( FMeshDescription& MeshDescription)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshDescriptionBulkData::LoadMeshDescription);

	MeshDescription.Empty();

	if (BulkData.HasPayloadData())
	{
#if WITH_EDITOR
		// TODO: Remove this once VirtualizedBulkData can be shown to be thread safe
		// A lock is required so we can clone the mesh description from multiple threads
		FRWScopeLock ScopeLock(BulkDataLock, SLT_Write);
#endif //WITH_EDITOR

		constexpr bool bIsPersistent = true;
		UE::Serialization::FEditorBulkDataReader Ar(BulkData, bIsPersistent);
	
		// Only attempt to load the mesh description if the payload actually exists. HasPayloadData only checks if the payload size is greater
		// than zero, not that the payload actually got loaded (which can happen asynchronously). 
		
		if (Ar.IsValid())
		{
			// Propagate the version information from the package to the bulk data, so that the MeshDescription
			// is serialized with the same versioning.
			Ar.SetUEVer(UEVersion);
			Ar.SetLicenseeUEVer(LicenseeUEVersion);
			Ar.SetCustomVersions(CustomVersions);
			Ar << MeshDescription;
		}

		BulkData.UnloadData();
	}
}

void FMeshDescriptionBulkData::UpdateMeshDescriptionFormat()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshDescriptionBulkData::UpdateMeshDescriptionFormat);
	if (bBulkDataUpdated)
	{
		return;
	}

	if (!BulkData.HasPayloadData())
	{
		return;
	}

#if WITH_EDITOR
	// TODO: Remove this once VirtualizedBulkData can be shown to be thread safe
	// A lock is required so we can clone the mesh description from multiple threads
	FRWScopeLock ScopeLock(BulkDataLock, SLT_Write);
#endif //WITH_EDITOR

	FSharedBuffer OldBytes = BulkData.GetPayload().Get();
	FMemoryReaderView Reader(OldBytes.GetView(), true /* bIsPersistent */);
	BulkData.UnloadData();
	// Propagate the custom version information from the package to the bulk data, so that the MeshDescription
	// is serialized with the same versioning.
	Reader.SetUEVer(UEVersion);
	Reader.SetLicenseeUEVer(LicenseeUEVersion);
	Reader.SetCustomVersions(CustomVersions);
	FMeshDescription MeshDescription;
	MeshDescription.Empty();
	Reader << MeshDescription;

	FLargeMemoryWriter NewBytes(OldBytes.GetSize(), true /* bIsPersistent */);
	if (!MeshDescription.IsEmpty())
	{
		NewBytes << MeshDescription;
	}
	uint64 NumBytes = static_cast<uint64>(NewBytes.TotalSize());

	if (NumBytes != OldBytes.GetSize() ||
		0 != FMemory::Memcmp(OldBytes.GetData(), NewBytes.GetData(), NumBytes))
	{
		// The MeshDescription format has changed, so save it to this's BulkData
		BulkData.UpdatePayload(FSharedBuffer::TakeOwnership(NewBytes.ReleaseOwnership(), NumBytes, FMemory::Free));
		// Preserve versions at save time so we can reuse the same ones when reloading direct from memory
		UEVersion = NewBytes.UEVer();
		LicenseeUEVersion = NewBytes.LicenseeUEVer();
		CustomVersions = NewBytes.GetCustomVersions();
		if (bGuidIsHash)
		{
			UseHashAsGuid();
		}
		else
		{
			// Maintain the original guid; this is a format change only. GetIdString will change because it adds the version information to the key.
		}
	}

	// Mark the MeshDescriptionBulkData as having been updated.
	// This means we know that its version is up-to-date.
	bBulkDataUpdated = true;
}

void FMeshDescriptionBulkData::Empty()
{
	BulkData.Reset();
}

FString FMeshDescriptionBulkData::GetIdStringFromVersionList(TConstArrayView<TPair<FGuid, int32>> InVersionList) const
{
	UE::DerivedData::FBuildVersionBuilder Builder;
	Builder << const_cast<FGuid&>(Guid);
	Builder << GPackageFileUEVersion;
	Builder << GPackageFileLicenseeUEVersion;
	for (const TPair<FGuid, int32>& Version : InVersionList)
	{
		Builder << const_cast<FGuid&>(Version.Key);
		Builder << Version.Value;
	}

	FString GuidString = Builder.Build().ToString();
	if (bGuidIsHash)
	{
		GuidString += TEXT("X");
	}
	return GuidString;
}

FGuid FMeshDescriptionBulkData::GetHash() const
{
	if (BulkData.HasPayloadData())
	{
		return UE::Serialization::IoHashToGuid(BulkData.GetPayloadId());
	}

	return FGuid();
}

FString FMeshDescriptionBulkData::GetSerializationVersionIdString()
{
	UE::DerivedData::FBuildVersionBuilder Builder;
	Builder << GPackageFileUEVersion;
	Builder << GPackageFileLicenseeUEVersion;
	for (const TPair<FGuid, int32>& Version : FMeshDescription::GetSerializationFormatVersions())
	{
		Builder << const_cast<FGuid&>(Version.Key);
		Builder << Version.Value;
	}

	return Builder.Build().ToString();
}

TConstArrayView<TPair<FGuid, int32>> FMeshDescriptionBulkData::GetFrozenVersions()
{
	// These versions are frozen and will never change.
	// DO NOT ALTER THIS LIST OR DDC READS WILL GET UNNECESSARILY INVALIDATED.
	static TPair<FGuid, int32> Versions[] {
			{ FEditorObjectVersion::GUID, FEditorObjectVersion::SkeletalMeshSourceDataSupport16bitOfMaterialNumber },
			{ FReleaseObjectVersion::GUID, FReleaseObjectVersion::GeometryCollectionCacheRemovesMassToLocal },
			{ FEnterpriseObjectVersion::GUID, FEnterpriseObjectVersion::AddedParametricSurfaceData },
			{ FUE5MainStreamObjectVersion::GUID, FUE5MainStreamObjectVersion::RuntimeVirtualTextureMaterialValidation },
		};
	return MakeArrayView(Versions); 
}

void FMeshDescriptionBulkData::UseHashAsGuid()
{
	if (BulkData.HasPayloadData())
	{
		bGuidIsHash = true;

		Guid = GetHash();
	}
	else
	{
		Guid.Invalidate();
	}	
}

#endif // #if WITH_EDITORONLY_DATA


