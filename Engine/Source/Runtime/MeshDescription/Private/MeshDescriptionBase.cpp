// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshDescriptionBase.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Algo/Copy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshDescriptionBase)



void UMeshDescriptionBase::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	Super::Serialize(Ar);

	if (!Ar.IsObjectReferenceCollector())
	{
		if (!Ar.IsLoading() || Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::SerializeMeshDescriptionBase)
		{
			Ar << OwnedMeshDescription;
		}
	}
}

void UMeshDescriptionBase::RegisterAttributes()
{
	RequiredAttributes = MakeUnique<FMeshAttributes>(GetMeshDescription());
	RequiredAttributes->Register();
}

void UMeshDescriptionBase::Reset()
{
	OwnedMeshDescription = FMeshDescription();
	RegisterAttributes();
}

void UMeshDescriptionBase::Empty()
{
	GetMeshDescription().Empty();
}

bool UMeshDescriptionBase::IsEmpty() const
{
	return GetMeshDescription().IsEmpty();
}

void UMeshDescriptionBase::ReserveNewVertices(int32 NumberOfNewVertices)
{
	GetMeshDescription().ReserveNewVertices(NumberOfNewVertices);
}

FVertexID UMeshDescriptionBase::CreateVertex()
{
	return GetMeshDescription().CreateVertex();
}

void UMeshDescriptionBase::CreateVertexWithID(FVertexID VertexID)
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		GetMeshDescription().CreateVertexWithID(VertexID);
	}
	else
	{
		UE_LOGF(LogMeshDescription, Warning, "CreateVertexWithID: VertexID %d already exists.", VertexID.GetValue());
	}
}

void UMeshDescriptionBase::DeleteVertex(FVertexID VertexID)
{
	if (GetMeshDescription().IsVertexValid(VertexID))
	{
		GetMeshDescription().DeleteVertex(VertexID);
	}
	else
	{
		UE_LOGF(LogMeshDescription, Warning, "DeleteVertex: VertexID %d doesn't exist.", VertexID.GetValue());
	}
}

bool UMeshDescriptionBase::IsVertexValid(FVertexID VertexID) const
{
	return GetMeshDescription().IsVertexValid(VertexID);
}

void UMeshDescriptionBase::ReserveNewVertexInstances(int32 NumberOfNewVertexInstances)
{
	GetMeshDescription().ReserveNewEdges(NumberOfNewVertexInstances);
}

FVertexInstanceID UMeshDescriptionBase::CreateVertexInstance(FVertexID VertexID)
{
	if (GetMeshDescription().IsVertexValid(VertexID))
	{
		return GetMeshDescription().CreateVertexInstance(VertexID);
	}
	else
	{
		UE_LOGF(LogMeshDescription, Warning, "CreateVertexInstance: VertexID %d doesn't exist.", VertexID.GetValue());
		return INDEX_NONE;
	}
}

void UMeshDescriptionBase::CreateVertexInstanceWithID(FVertexInstanceID VertexInstanceID, FVertexID VertexID)
{
	if (GetMeshDescription().IsVertexValid(VertexID))
	{
		if (!GetMeshDescription().IsVertexInstanceValid(VertexInstanceID))
		{
			GetMeshDescription().CreateVertexInstanceWithID(VertexInstanceID, VertexID);
		}
		else
		{
			UE_LOGF(LogMeshDescription, Warning, "CreateVertexInstanceWithID: VertexInstanceID %d already exists.", VertexInstanceID.GetValue());
		}
	}
	else
	{
		UE_LOGF(LogMeshDescription, Warning, "CreateVertexInstanceWithID: VertexID %d doesn't exist.", VertexID.GetValue());
	}
}

void UMeshDescriptionBase::DeleteVertexInstance(FVertexInstanceID VertexInstanceID, TArray<FVertexID>& OrphanedVertices)
{
	if (GetMeshDescription().IsVertexInstanceValid(VertexInstanceID))
	{
		GetMeshDescription().DeleteVertexInstance(VertexInstanceID, &OrphanedVertices);
	}
	else
	{
		UE_LOGF(LogMeshDescription, Warning, "DeleteVertexInstance: VertexInstanceID %d doesn't exist.", VertexInstanceID.GetValue());
	}
}

bool UMeshDescriptionBase::IsVertexInstanceValid(FVertexInstanceID VertexInstanceID) const
{
	return GetMeshDescription().IsVertexInstanceValid(VertexInstanceID);
}

void UMeshDescriptionBase::ReserveNewEdges(int32 NumberOfNewEdges)
{
	GetMeshDescription().ReserveNewEdges(NumberOfNewEdges);
}

FEdgeID UMeshDescriptionBase::CreateEdge(FVertexID VertexID0, FVertexID VertexID1)
{
	if (!GetMeshDescription().IsVertexValid(VertexID0))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreateEdge: VertexID %d doesn't exist.", VertexID0.GetValue());
		return INDEX_NONE;
	}

	if (!GetMeshDescription().IsVertexValid(VertexID1))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreateEdge: VertexID %d doesn't exist.", VertexID1.GetValue());
		return INDEX_NONE;
	}

	return GetMeshDescription().CreateEdge(VertexID0, VertexID1);
}

void UMeshDescriptionBase::CreateEdgeWithID(FEdgeID EdgeID, FVertexID VertexID0, FVertexID VertexID1)
{
	if (GetMeshDescription().IsEdgeValid(EdgeID))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreateEdgeWithID: EdgeID %d already exists.", EdgeID.GetValue());
		return;
	}

	if (!GetMeshDescription().IsVertexValid(VertexID0))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreateEdgeWithID: VertexID %d doesn't exist.", VertexID0.GetValue());
		return;
	}

	if (!GetMeshDescription().IsVertexValid(VertexID1))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreateEdgeWithID: VertexID %d doesn't exist.", VertexID1.GetValue());
		return;
	}

	GetMeshDescription().CreateEdgeWithID(EdgeID, VertexID0, VertexID1);
}

void UMeshDescriptionBase::DeleteEdge(FEdgeID EdgeID, TArray<FVertexID>& OrphanedVertices)
{
	if (GetMeshDescription().IsEdgeValid(EdgeID))
	{
		GetMeshDescription().DeleteEdge(EdgeID, &OrphanedVertices);
	}
	else
	{
		UE_LOGF(LogMeshDescription, Warning, "DeleteEdge: EdgeID %d doesn't exist.", EdgeID.GetValue());
	}
}

bool UMeshDescriptionBase::IsEdgeValid(FEdgeID EdgeID) const
{
	return GetMeshDescription().IsEdgeValid(EdgeID);
}

void UMeshDescriptionBase::ReserveNewTriangles(int32 NumberOfNewTriangles)
{
	GetMeshDescription().ReserveNewTriangles(NumberOfNewTriangles);
}

FTriangleID UMeshDescriptionBase::CreateTriangle(FPolygonGroupID PolygonGroupID, const TArray<FVertexInstanceID>& VertexInstanceIDs, TArray<FEdgeID>& NewEdgeIDs)
{
	if (!GetMeshDescription().IsPolygonGroupValid(PolygonGroupID))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreateTriangle: PolygonGroupID %d doesn't exist.", PolygonGroupID.GetValue());
		return INDEX_NONE;
	}

	return GetMeshDescription().CreateTriangle(PolygonGroupID, VertexInstanceIDs, &NewEdgeIDs);
}

void UMeshDescriptionBase::CreateTriangleWithID(FTriangleID TriangleID, FPolygonGroupID PolygonGroupID, const TArray<FVertexInstanceID>& VertexInstanceIDs, TArray<FEdgeID>& NewEdgeIDs)
{
	if (GetMeshDescription().IsTriangleValid(TriangleID))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreateTriangleWithID: TriangleID %d already exists.", TriangleID.GetValue());
		return;
	}

	if (!GetMeshDescription().IsPolygonGroupValid(PolygonGroupID))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreateTriangleWithID: PolygonGroupID %d doesn't exist.", PolygonGroupID.GetValue());
		return;
	}

	GetMeshDescription().CreateTriangleWithID(TriangleID, PolygonGroupID, VertexInstanceIDs, &NewEdgeIDs);
}

void UMeshDescriptionBase::DeleteTriangle(FTriangleID TriangleID, TArray<FEdgeID>& OrphanedEdges, TArray<FVertexInstanceID>& OrphanedVertexInstances, TArray<FPolygonGroupID>& OrphanedPolygonGroups)
{
	if (!GetMeshDescription().IsTriangleValid(TriangleID))
	{
		UE_LOGF(LogMeshDescription, Warning, "DeleteTriangle: TriangleID %d doesn't exist.", TriangleID.GetValue());
		return;
	}

	GetMeshDescription().DeleteTriangle(TriangleID, &OrphanedEdges, &OrphanedVertexInstances, &OrphanedPolygonGroups);
}

bool UMeshDescriptionBase::IsTriangleValid(const FTriangleID TriangleID) const
{
	return GetMeshDescription().IsTriangleValid(TriangleID);
}

void UMeshDescriptionBase::ReserveNewPolygons(const int32 NumberOfNewPolygons)
{
	GetMeshDescription().ReserveNewPolygons(NumberOfNewPolygons);
}

FPolygonID UMeshDescriptionBase::CreatePolygon(FPolygonGroupID PolygonGroupID, TArray<FVertexInstanceID>& VertexInstanceIDs, TArray<FEdgeID>& NewEdgeIDs)
{
	if (!GetMeshDescription().IsPolygonGroupValid(PolygonGroupID))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreatePolygon: PolygonGroupID %d doesn't exist.", PolygonGroupID.GetValue());
		return INDEX_NONE;
	}

	return GetMeshDescription().CreatePolygon(PolygonGroupID, VertexInstanceIDs, &NewEdgeIDs);
}

void UMeshDescriptionBase::CreatePolygonWithID(FPolygonID PolygonID, FPolygonGroupID PolygonGroupID, TArray<FVertexInstanceID>& VertexInstanceIDs, TArray<FEdgeID>& NewEdgeIDs)
{
	if (GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreatePolygonWithID: PolygonID %d already exists.", PolygonID.GetValue());
		return;
	}

	if (!GetMeshDescription().IsPolygonGroupValid(PolygonGroupID))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreatePolygonWithID: PolygonGroupID %d doesn't exist.", PolygonGroupID.GetValue());
		return;
	}

	GetMeshDescription().CreatePolygonWithID(PolygonID, PolygonGroupID, VertexInstanceIDs, &NewEdgeIDs);
}

void UMeshDescriptionBase::DeletePolygon(FPolygonID PolygonID, TArray<FEdgeID>& OrphanedEdges, TArray<FVertexInstanceID>& OrphanedVertexInstances, TArray<FPolygonGroupID>& OrphanedPolygonGroups)
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "DeletePolygon: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return;
	}

	GetMeshDescription().DeletePolygon(PolygonID, &OrphanedEdges, &OrphanedVertexInstances, &OrphanedPolygonGroups);
}

bool UMeshDescriptionBase::IsPolygonValid(FPolygonID PolygonID) const
{
	return GetMeshDescription().IsPolygonValid(PolygonID);
}

void UMeshDescriptionBase::ReserveNewPolygonGroups(int32 NumberOfNewPolygonGroups)
{
	GetMeshDescription().ReserveNewPolygonGroups(NumberOfNewPolygonGroups);
}

FPolygonGroupID UMeshDescriptionBase::CreatePolygonGroup()
{
	return GetMeshDescription().CreatePolygonGroup();
}

void UMeshDescriptionBase::CreatePolygonGroupWithID(FPolygonGroupID PolygonGroupID)
{
	if (GetMeshDescription().IsPolygonGroupValid(PolygonGroupID))
	{
		UE_LOGF(LogMeshDescription, Warning, "CreatePolygonGroupWithID: PolygonGroupID %d already exists.", PolygonGroupID.GetValue());
		return;
	}

	GetMeshDescription().CreatePolygonGroupWithID(PolygonGroupID);
}

void UMeshDescriptionBase::DeletePolygonGroup(FPolygonGroupID PolygonGroupID)
{
	if (!GetMeshDescription().IsPolygonGroupValid(PolygonGroupID))
	{
		UE_LOGF(LogMeshDescription, Warning, "DeletePolygonGroup: FPolygonGroupID %d doesn't exist.", PolygonGroupID.GetValue());
		return;
	}

	GetMeshDescription().DeletePolygonGroup(PolygonGroupID);
}

bool UMeshDescriptionBase::IsPolygonGroupValid(FPolygonGroupID PolygonGroupID) const
{
	return GetMeshDescription().IsPolygonGroupValid(PolygonGroupID);
}


//////////////////////////////////////////////////////////////////////
// Vertex operations

bool UMeshDescriptionBase::IsVertexOrphaned(FVertexID VertexID) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "IsVertexOrphaned: VertexID %d doesn't exist.", VertexID.GetValue());
		return false;
	}

	return GetMeshDescription().IsVertexOrphaned(VertexID);
}

FEdgeID UMeshDescriptionBase::GetVertexPairEdge(FVertexID VertexID0, FVertexID VertexID1) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID0))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexPairEdge: VertexID %d doesn't exist.", VertexID0.GetValue());
		return INDEX_NONE;
	}

	if (!GetMeshDescription().IsVertexValid(VertexID1))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexPairEdge: VertexID %d doesn't exist.", VertexID1.GetValue());
		return INDEX_NONE;
	}

	return GetMeshDescription().GetVertexPairEdge(VertexID0, VertexID1);
}

void UMeshDescriptionBase::GetVertexConnectedEdges(FVertexID VertexID, TArray<FEdgeID>& OutEdgeIDs) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexConnectedEdges: VertexID %d doesn't exist.", VertexID.GetValue());
		return;
	}

	OutEdgeIDs = GetMeshDescription().GetVertexConnectedEdgeIDs(VertexID);
}

int32 UMeshDescriptionBase::GetNumVertexConnectedEdges(FVertexID VertexID) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumVertexConnectedEdges: VertexID %d doesn't exist.", VertexID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumVertexConnectedEdges(VertexID);
}

void UMeshDescriptionBase::GetVertexVertexInstances(FVertexID VertexID, TArray<FVertexInstanceID>& OutVertexInstanceIDs) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexVertexInstances: VertexID %d doesn't exist.", VertexID.GetValue());
		return;
	}

	OutVertexInstanceIDs = GetMeshDescription().GetVertexVertexInstanceIDs(VertexID);
}

int32 UMeshDescriptionBase::GetNumVertexVertexInstances(FVertexID VertexID) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumVertexVertexInstances: VertexID %d doesn't exist.", VertexID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumVertexVertexInstances(VertexID);
}

void UMeshDescriptionBase::GetVertexConnectedTriangles(FVertexID VertexID, TArray<FTriangleID>& OutConnectedTriangleIDs) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexConnectedTriangles: VertexID %d doesn't exist.", VertexID.GetValue());
		return;
	}

	GetMeshDescription().GetVertexConnectedTriangles(VertexID, OutConnectedTriangleIDs);
}

int32 UMeshDescriptionBase::GetNumVertexConnectedTriangles(FVertexID VertexID) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumVertexConnectedTriangles: VertexID %d doesn't exist.", VertexID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumVertexConnectedTriangles(VertexID);
}

void UMeshDescriptionBase::GetVertexConnectedPolygons(FVertexID VertexID, TArray<FPolygonID>& OutConnectedPolygonIDs) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexConnectedPolygons: VertexID %d doesn't exist.", VertexID.GetValue());
		return;
	}

	GetMeshDescription().GetVertexConnectedPolygons(VertexID, OutConnectedPolygonIDs);
}

int32 UMeshDescriptionBase::GetNumVertexConnectedPolygons(FVertexID VertexID) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumVertexConnectedPolygons: VertexID %d doesn't exist.", VertexID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumVertexConnectedPolygons(VertexID);
}

void UMeshDescriptionBase::GetVertexAdjacentVertices(FVertexID VertexID, TArray<FVertexID>& OutAdjacentVertexIDs) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexAdjacentVertices: VertexID %d doesn't exist.", VertexID.GetValue());
		return;
	}

	GetMeshDescription().GetVertexAdjacentVertices(VertexID, OutAdjacentVertexIDs);
}

FVector UMeshDescriptionBase::GetVertexPosition(FVertexID VertexID) const
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexAttribute: VertexID %d doesn't exist.", VertexID.GetValue());
		return FVector::ZeroVector;
	}

	if (!GetMeshDescription().VertexAttributes().HasAttribute(MeshAttribute::Vertex::Position))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexAttribute: VertexAttribute Position doesn't exist.");
		return FVector::ZeroVector;
	}

	return FVector(GetMeshDescription().VertexAttributes().GetAttribute<FVector3f>(VertexID, MeshAttribute::Vertex::Position));
}

void UMeshDescriptionBase::SetVertexPosition(FVertexID VertexID, const FVector& Position)
{
	if (!GetMeshDescription().IsVertexValid(VertexID))
	{
		UE_LOGF(LogMeshDescription, Warning, "SetVertexAttribute: VertexID %d doesn't exist.", VertexID.GetValue());
		return;
	}

	if (!GetMeshDescription().VertexAttributes().HasAttribute(MeshAttribute::Vertex::Position))
	{
		UE_LOGF(LogMeshDescription, Warning, "SetVertexAttribute: VertexAttribute Position doesn't exist.");
		return;
	}

	GetMeshDescription().VertexAttributes().SetAttribute(VertexID, MeshAttribute::Vertex::Position, 0, FVector3f(Position));
}


//////////////////////////////////////////////////////////////////////
// Vertex instance operations

FVertexID UMeshDescriptionBase::GetVertexInstanceVertex(FVertexInstanceID VertexInstanceID) const
{
	if (!GetMeshDescription().IsVertexInstanceValid(VertexInstanceID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexInstanceVertex: VertexInstanceID %d doesn't exist.", VertexInstanceID.GetValue());
		return INDEX_NONE;
	}

	return GetMeshDescription().GetVertexInstanceVertex(VertexInstanceID);
}

FEdgeID UMeshDescriptionBase::GetVertexInstancePairEdge(FVertexInstanceID VertexInstanceID0, FVertexInstanceID VertexInstanceID1) const
{
	if (!GetMeshDescription().IsVertexInstanceValid(VertexInstanceID0))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexInstancePairEdge: VertexInstanceID %d doesn't exist.", VertexInstanceID0.GetValue());
		return INDEX_NONE;
	}

	if (!GetMeshDescription().IsVertexInstanceValid(VertexInstanceID1))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexInstancePairEdge: VertexInstanceID %d doesn't exist.", VertexInstanceID1.GetValue());
		return INDEX_NONE;
	}

	return GetMeshDescription().GetVertexInstancePairEdge(VertexInstanceID0, VertexInstanceID1);
}

void UMeshDescriptionBase::GetVertexInstanceConnectedTriangles(FVertexInstanceID VertexInstanceID, TArray<FTriangleID>& OutConnectedTriangleIDs) const
{
	if (!GetMeshDescription().IsVertexInstanceValid(VertexInstanceID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexInstanceConnectedTriangles: VertexInstanceID %d doesn't exist.", VertexInstanceID.GetValue());
		return;
	}

	OutConnectedTriangleIDs = GetMeshDescription().GetVertexInstanceConnectedTriangleIDs(VertexInstanceID);
}

int32 UMeshDescriptionBase::GetNumVertexInstanceConnectedTriangles(FVertexInstanceID VertexInstanceID) const
{
	if (!GetMeshDescription().IsVertexInstanceValid(VertexInstanceID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumVertexInstanceConnectedTriangles: VertexInstanceID %d doesn't exist.", VertexInstanceID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumVertexInstanceConnectedTriangles(VertexInstanceID);
}

void UMeshDescriptionBase::GetVertexInstanceConnectedPolygons(FVertexInstanceID VertexInstanceID, TArray<FPolygonID>& OutConnectedPolygonIDs) const
{
	if (!GetMeshDescription().IsVertexInstanceValid(VertexInstanceID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexInstanceConnectedPolygons: VertexInstanceID %d doesn't exist.", VertexInstanceID.GetValue());
		return;
	}

	GetMeshDescription().GetVertexInstanceConnectedPolygons(VertexInstanceID, OutConnectedPolygonIDs);
}

int32 UMeshDescriptionBase::GetNumVertexInstanceConnectedPolygons(FVertexInstanceID VertexInstanceID) const
{
	if (!GetMeshDescription().IsVertexInstanceValid(VertexInstanceID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumVertexInstanceConnectedPolygons: VertexInstanceID %d doesn't exist.", VertexInstanceID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumVertexInstanceConnectedPolygons(VertexInstanceID);
}


//////////////////////////////////////////////////////////////////////
// Edge operations

bool UMeshDescriptionBase::IsEdgeInternal(FEdgeID EdgeID) const
{
	if (!GetMeshDescription().IsEdgeValid(EdgeID))
	{
		UE_LOGF(LogMeshDescription, Warning, "IsEdgeInternal: EdgeID %d doesn't exist.", EdgeID.GetValue());
		return false;
	}

	return GetMeshDescription().IsEdgeInternal(EdgeID);
}

bool UMeshDescriptionBase::IsEdgeInternalToPolygon(FEdgeID EdgeID, FPolygonID PolygonID) const
{
	if (!GetMeshDescription().IsEdgeValid(EdgeID))
	{
		UE_LOGF(LogMeshDescription, Warning, "IsEdgeInternalToPolygon: EdgeID %d doesn't exist.", EdgeID.GetValue());
		return false;
	}

	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "IsEdgeInternalToPolygon: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return false;
	}

	return GetMeshDescription().IsEdgeInternalToPolygon(EdgeID, PolygonID);
}

void UMeshDescriptionBase::GetEdgeConnectedTriangles(FEdgeID EdgeID, TArray<FTriangleID>& OutConnectedTriangleIDs) const
{
	if (!GetMeshDescription().IsEdgeValid(EdgeID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetEdgeConnectedTriangles: EdgeID %d doesn't exist.", EdgeID.GetValue());
		return;
	}

	OutConnectedTriangleIDs = GetMeshDescription().GetEdgeConnectedTriangleIDs(EdgeID);
}

int32 UMeshDescriptionBase::GetNumEdgeConnectedTriangles(FEdgeID EdgeID) const
{
	if (!GetMeshDescription().IsEdgeValid(EdgeID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumEdgeConnectedTriangles: EdgeID %d doesn't exist.", EdgeID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumEdgeConnectedTriangles(EdgeID);
}

void UMeshDescriptionBase::GetEdgeConnectedPolygons(FEdgeID EdgeID, TArray<FPolygonID>& OutConnectedPolygonIDs) const
{
	if (!GetMeshDescription().IsEdgeValid(EdgeID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetEdgeConnectedPolygons: EdgeID %d doesn't exist.", EdgeID.GetValue());
		return;
	}

	GetMeshDescription().GetEdgeConnectedPolygons(EdgeID, OutConnectedPolygonIDs);
}

int32 UMeshDescriptionBase::GetNumEdgeConnectedPolygons(FEdgeID EdgeID) const
{
	if (!GetMeshDescription().IsEdgeValid(EdgeID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumEdgeConnectedPolygons: EdgeID %d doesn't exist.", EdgeID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumEdgeConnectedPolygons(EdgeID);
}

FVertexID UMeshDescriptionBase::GetEdgeVertex(FEdgeID EdgeID, int32 VertexNumber) const
{
	if (!GetMeshDescription().IsEdgeValid(EdgeID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetEdgeVertex: EdgeID %d doesn't exist.", EdgeID.GetValue());
		return INDEX_NONE;
	}

	if (VertexNumber != 0 && VertexNumber != 1)
	{
		UE_LOGF(LogMeshDescription, Warning, "GetEdgeVertex: invalid vertex number %d.", VertexNumber);
		return INDEX_NONE;
	}

	return GetMeshDescription().GetEdgeVertex(EdgeID, VertexNumber);
}

void UMeshDescriptionBase::GetEdgeVertices(const FEdgeID EdgeID, TArray<FVertexID>& OutVertexIDs) const
{
	if (!GetMeshDescription().IsEdgeValid(EdgeID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetEdgeVertices: EdgeID %d doesn't exist.", EdgeID.GetValue());
		return;
	}

	Algo::Copy(GetMeshDescription().GetEdgeVertices(EdgeID), OutVertexIDs);
}


//////////////////////////////////////////////////////////////////////
// Triangle operations

FPolygonID UMeshDescriptionBase::GetTrianglePolygon(FTriangleID TriangleID) const
{
	if (!GetMeshDescription().IsTriangleValid(TriangleID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetTrianglePolygon: TriangleID %d doesn't exist.", TriangleID.GetValue());
		return INDEX_NONE;
	}

	return GetMeshDescription().GetTrianglePolygon(TriangleID);
}

FPolygonGroupID UMeshDescriptionBase::GetTrianglePolygonGroup(FTriangleID TriangleID) const
{
	if (!GetMeshDescription().IsTriangleValid(TriangleID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetTrianglePolygonGroup: TriangleID %d doesn't exist.", TriangleID.GetValue());
		return INDEX_NONE;
	}

	return GetMeshDescription().GetTrianglePolygonGroup(TriangleID);
}

bool UMeshDescriptionBase::IsTrianglePartOfNgon(FTriangleID TriangleID) const
{
	if (!GetMeshDescription().IsTriangleValid(TriangleID))
	{
		UE_LOGF(LogMeshDescription, Warning, "IsTrianglePartOfNgon: TriangleID %d doesn't exist.", TriangleID.GetValue());
		return false;
	}

	return GetMeshDescription().IsTrianglePartOfNgon(TriangleID);
}

void UMeshDescriptionBase::GetTriangleVertexInstances(FTriangleID TriangleID, TArray<FVertexInstanceID>& OutVertexInstanceIDs) const
{
	if (!GetMeshDescription().IsTriangleValid(TriangleID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetTriangleVertexInstances: TriangleID %d doesn't exist.", TriangleID.GetValue());
		return;
	}

	Algo::Copy(GetMeshDescription().GetTriangleVertexInstances(TriangleID), OutVertexInstanceIDs);
}

FVertexInstanceID UMeshDescriptionBase::GetTriangleVertexInstance(FTriangleID TriangleID, int32 Index) const
{
	if (!GetMeshDescription().IsTriangleValid(TriangleID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetTriangleVertexInstance: TriangleID %d doesn't exist.", TriangleID.GetValue());
		return INDEX_NONE;
	}

	if (Index < 0 || Index > 2)
	{
		UE_LOGF(LogMeshDescription, Warning, "GetTriangleVertexInstance: invalid vertex index %d.", Index);
		return INDEX_NONE;
	}

	return GetMeshDescription().GetTriangleVertexInstance(TriangleID, Index);
}

void UMeshDescriptionBase::GetTriangleVertices(FTriangleID TriangleID, TArray<FVertexID>& OutVertexIDs) const
{
	if (!GetMeshDescription().IsTriangleValid(TriangleID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetTriangleVertices: TriangleID %d doesn't exist.", TriangleID.GetValue());
		return;
	}

	OutVertexIDs.SetNumUninitialized(3);
	Algo::Copy(GetMeshDescription().GetTriangleVertices(TriangleID), OutVertexIDs);
}

void UMeshDescriptionBase::GetTriangleEdges(FTriangleID TriangleID, TArray<FEdgeID>& OutEdgeIDs) const
{
	if (!GetMeshDescription().IsTriangleValid(TriangleID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetTriangleEdges: TriangleID %d doesn't exist.", TriangleID.GetValue());
		return;
	}

	OutEdgeIDs.SetNumUninitialized(3);
	Algo::Copy(GetMeshDescription().GetTriangleEdges(TriangleID), OutEdgeIDs);
}

void UMeshDescriptionBase::GetTriangleAdjacentTriangles(FTriangleID TriangleID, TArray<FTriangleID>& OutTriangleIDs) const
{
	if (!GetMeshDescription().IsTriangleValid(TriangleID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetTriangleAdjacentTriangles: TriangleID %d doesn't exist.", TriangleID.GetValue());
		return;
	}

	GetMeshDescription().GetTriangleAdjacentTriangles(TriangleID, OutTriangleIDs);
}

FVertexInstanceID UMeshDescriptionBase::GetVertexInstanceForTriangleVertex(FTriangleID TriangleID, FVertexID VertexID) const
{
	if (!GetMeshDescription().IsTriangleValid(TriangleID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexInstanceForTriangleVertex: TriangleID %d doesn't exist.", TriangleID.GetValue());
		return INDEX_NONE;
	}

	return GetMeshDescription().GetVertexInstanceForTriangleVertex(TriangleID, VertexID);
}


//////////////////////////////////////////////////////////////////////
// Polygon operations

void UMeshDescriptionBase::GetPolygonTriangles(FPolygonID PolygonID, TArray<FTriangleID>& OutTriangleIDs) const
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetPolygonTriangles: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return;
	}

	OutTriangleIDs = GetMeshDescription().GetPolygonTriangles(PolygonID);
}

int32 UMeshDescriptionBase::GetNumPolygonTriangles(FPolygonID PolygonID) const
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumPolygonTriangles: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumPolygonTriangles(PolygonID);
}

void UMeshDescriptionBase::GetPolygonVertexInstances(FPolygonID PolygonID, TArray<FVertexInstanceID>& OutVertexInstanceIDs) const
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetPolygonVertexInstances: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return;
	}

	OutVertexInstanceIDs = GetMeshDescription().GetPolygonVertexInstances(PolygonID);
}

int32 UMeshDescriptionBase::GetNumPolygonVertices(FPolygonID PolygonID) const
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumPolygonVertices: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumPolygonVertices(PolygonID);
}

void UMeshDescriptionBase::GetPolygonVertices(FPolygonID PolygonID, TArray<FVertexID>& OutVertexIDs) const
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetPolygonVertices: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return;
	}

	GetMeshDescription().GetPolygonVertices(PolygonID, OutVertexIDs);
}

void UMeshDescriptionBase::GetPolygonPerimeterEdges(FPolygonID PolygonID, TArray<FEdgeID>& OutEdgeIDs) const
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetPolygonPerimeterEdges: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return;
	}

	GetMeshDescription().GetPolygonPerimeterEdges(PolygonID, OutEdgeIDs);
}

void UMeshDescriptionBase::GetPolygonInternalEdges(FPolygonID PolygonID, TArray<FEdgeID>& OutEdgeIDs) const
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetPolygonInternalEdges: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return;
	}

	GetMeshDescription().GetPolygonInternalEdges(PolygonID, OutEdgeIDs);
}

int32 UMeshDescriptionBase::GetNumPolygonInternalEdges(FPolygonID PolygonID) const
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumPolygonInternalEdges: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumPolygonInternalEdges(PolygonID);
}

void UMeshDescriptionBase::GetPolygonAdjacentPolygons(FPolygonID PolygonID, TArray<FPolygonID>& OutPolygonIDs) const
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetPolygonAdjacentPolygons: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return;
	}

	GetMeshDescription().GetPolygonAdjacentPolygons(PolygonID, OutPolygonIDs);
}

FPolygonGroupID UMeshDescriptionBase::GetPolygonPolygonGroup(FPolygonID PolygonID) const
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetPolygonPolygonGroup: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return INDEX_NONE;
	}

	return GetMeshDescription().GetPolygonPolygonGroup(PolygonID);
}

FVertexInstanceID UMeshDescriptionBase::GetVertexInstanceForPolygonVertex(FPolygonID PolygonID, FVertexID VertexID) const
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexInstanceForPolygonVertex: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return INDEX_NONE;
	}

	return GetMeshDescription().GetVertexInstanceForPolygonVertex(PolygonID, VertexID);
}

void UMeshDescriptionBase::SetPolygonVertexInstances(FPolygonID PolygonID, const TArray<FVertexInstanceID>& VertexInstanceIDs)
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "SetPolygonVertexInstances: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return;
	}

	for (FVertexInstanceID VertexInstanceID : VertexInstanceIDs)
	{
		if (!GetMeshDescription().IsVertexInstanceValid(VertexInstanceID))
		{
			UE_LOGF(LogMeshDescription, Warning, "SetPolygonVertexInstances: VertexInstanceID %d doesn't exist.", VertexInstanceID.GetValue());
			return;
		}
	}

	GetMeshDescription().SetPolygonVertexInstances(PolygonID, VertexInstanceIDs);
}

void UMeshDescriptionBase::SetPolygonPolygonGroup(FPolygonID PolygonID, FPolygonGroupID PolygonGroupID)
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "SetPolygonPolygonGroup: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return;
	}

	if (!GetMeshDescription().IsPolygonGroupValid(PolygonGroupID))
	{
		UE_LOGF(LogMeshDescription, Warning, "SetPolygonPolygonGroup: PolygonGroupID %d doesn't exist.", PolygonGroupID.GetValue());
		return;
	}

	GetMeshDescription().SetPolygonPolygonGroup(PolygonID, PolygonGroupID);
}

void UMeshDescriptionBase::ReversePolygonFacing(FPolygonID PolygonID)
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "ReversePolygonFacing: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return;
	}

	GetMeshDescription().ReversePolygonFacing(PolygonID);
}

void UMeshDescriptionBase::ComputePolygonTriangulation(FPolygonID PolygonID)
{
	if (!GetMeshDescription().IsPolygonValid(PolygonID))
	{
		UE_LOGF(LogMeshDescription, Warning, "ComputePolygonTriangulation: PolygonID %d doesn't exist.", PolygonID.GetValue());
		return;
	}

	GetMeshDescription().ComputePolygonTriangulation(PolygonID);
}


//////////////////////////////////////////////////////////////////////
// Polygon group operations

void UMeshDescriptionBase::GetPolygonGroupPolygons(FPolygonGroupID PolygonGroupID, TArray<FPolygonID>& OutPolygonIDs) const
{
	if (!GetMeshDescription().IsPolygonGroupValid(PolygonGroupID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetPolygonGroupPolygons: PolygonGroupID %d doesn't exist.", PolygonGroupID.GetValue());
		return;
	}

	OutPolygonIDs = GetMeshDescription().GetPolygonGroupPolygonIDs(PolygonGroupID);
}

int32 UMeshDescriptionBase::GetNumPolygonGroupPolygons(FPolygonGroupID PolygonGroupID) const
{
	if (!GetMeshDescription().IsPolygonGroupValid(PolygonGroupID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetNumPolygonGroupPolygons: PolygonGroupID %d doesn't exist.", PolygonGroupID.GetValue());
		return 0;
	}

	return GetMeshDescription().GetNumPolygonGroupPolygons(PolygonGroupID);
}

