// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshDescriptionBaseBulkData.h"
#include "MeshDescriptionBase.h"
#include "MeshDescription.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshDescriptionBaseBulkData)


UMeshDescriptionBaseBulkData::UMeshDescriptionBaseBulkData()
{
#if WITH_EDITORONLY_DATA
	BulkData = MakePimpl<FMeshDescriptionBulkData>();
	MeshDescription = nullptr;
#endif
}


void UMeshDescriptionBaseBulkData::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	Super::Serialize(Ar);

	if (!Ar.IsFilterEditorOnly())
	{
		check(BulkData.IsValid());
		BulkData->Serialize(Ar, GetOuter());
	}

	if (Ar.IsLoading() && MeshDescription != nullptr)
	{
		// If there was a cached mesh description, it could be out of sync with the bulk data we just serialized so re-cache it.
		CacheMeshDescription();
	}
#endif
}


bool UMeshDescriptionBaseBulkData::IsEditorOnly() const
{
	return true;
}


bool UMeshDescriptionBaseBulkData::NeedsLoadForClient() const
{
	return false;
}


bool UMeshDescriptionBaseBulkData::NeedsLoadForServer() const
{
	return false;
}


bool UMeshDescriptionBaseBulkData::NeedsLoadForEditorGame() const
{
	return true;
}


#if WITH_EDITORONLY_DATA

void UMeshDescriptionBaseBulkData::Empty()
{
	BulkData->Empty();
}


UMeshDescriptionBase* UMeshDescriptionBaseBulkData::CreateMeshDescription()
{
	if (MeshDescription == nullptr)
	{
		check(PreallocatedMeshDescription);
		MeshDescription = PreallocatedMeshDescription;
	}

	// Reset the mesh description and register its attributes
	// Do this instead of always creating a new UStaticMeshDescription object, to save memory and improve performance
	MeshDescription->Reset();

	return MeshDescription;
}


UMeshDescriptionBase* UMeshDescriptionBaseBulkData::GetMeshDescription() const
{
	return MeshDescription;
}


bool UMeshDescriptionBaseBulkData::HasCachedMeshDescription() const
{
	return (MeshDescription != nullptr);
}


bool UMeshDescriptionBaseBulkData::CacheMeshDescription()
{
	check(MeshDescription);

	if (!BulkData->IsEmpty())
	{
		FMeshDescription NewMeshDescription;
		BulkData->LoadMeshDescription(NewMeshDescription);
		MeshDescription->SetMeshDescription(MoveTemp(NewMeshDescription));
		return true;
	}
	else
	{
		RemoveMeshDescription();
		return false;
	}
}


void UMeshDescriptionBaseBulkData::CommitMeshDescription(bool bUseHashAsGuid)
{
	if (MeshDescription != nullptr)
	{
		BulkData->SaveMeshDescription(MeshDescription->GetMeshDescription());
		if (bUseHashAsGuid)
		{
			BulkData->UseHashAsGuid();
		}
	}
	else
	{
		BulkData->Empty();
	}
}


void UMeshDescriptionBaseBulkData::RemoveMeshDescription()
{
	if (MeshDescription != nullptr)
	{
		MeshDescription->Reset();
		MeshDescription = nullptr;
	}
}


bool UMeshDescriptionBaseBulkData::IsBulkDataValid() const
{
	return !BulkData->IsEmpty();
}


const FMeshDescriptionBulkData& UMeshDescriptionBaseBulkData::GetBulkData() const
{
	return *BulkData;
}


FMeshDescriptionBulkData& UMeshDescriptionBaseBulkData::GetBulkData()
{
	return *BulkData;
}

#endif

