// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMeshDescriptionModule.h"
#include "Modules/ModuleManager.h"

class FMeshDescriptionModule : public IMeshDescriptionModule
{
public:

	FMeshDescriptionModule()
	{
	}

	virtual void StartupModule() override
	{
		// Register any modular features here
	}

	virtual void ShutdownModule() override
	{
		// Unregister any modular features here
	}

private:

};

IMPLEMENT_MODULE( FMeshDescriptionModule, MeshDescription );
