// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshElementContainer.h"
#include "Algo/MaxElement.h"


void FMeshElementContainer::Compact(TSparseArray<int32>& OutIndexRemap)
{
	const int32 NumElements = BitArray.Num() - NumHoles;
	OutIndexRemap.Empty(BitArray.Num());

	int32 NewIndex = 0;
	for (TConstSetBitIterator<> It(BitArray); It; ++It)
	{
		OutIndexRemap.Insert(It.GetIndex(), NewIndex);
		NewIndex++;
	}

	BitArray = TBitArray<>(true, NumElements);
	NumHoles = 0;

	Attributes.Remap(OutIndexRemap);
}


void FMeshElementContainer::Remap(const TSparseArray<int32>& IndexRemap)
{
	check(IndexRemap.Num() == BitArray.Num() - NumHoles);
	const int32 MaxNewIndex = *Algo::MaxElement(IndexRemap);

	TBitArray<> NewBitArray(false, MaxNewIndex + 1);
	for (TConstSetBitIterator<> It(BitArray); It; ++It)
	{
		const int32 OldIndex = It.GetIndex();
		const int32 NewIndex = IndexRemap[OldIndex];
		check(!NewBitArray[NewIndex]);
		NewBitArray[NewIndex] = true;
	}

	BitArray = MoveTemp(NewBitArray);
	NumHoles = BitArray.Num() - IndexRemap.Num();

	Attributes.Remap(IndexRemap);
}
