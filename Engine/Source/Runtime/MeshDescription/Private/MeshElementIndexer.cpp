// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshElementIndexer.h"
#include "MeshElementContainer.h"
#include "Algo/BinarySearch.h"


FMeshElementIndexer::FMeshElementIndexer(const FMeshElementChannels* InKey, const FMeshElementChannels* Referencer, FName AttributeName, int32 ReferencerChannelIndex)
{
	Set(InKey, Referencer, AttributeName, ReferencerChannelIndex);
}


void FMeshElementIndexer::Set(const FMeshElementChannels* InKey, const FMeshElementChannels* Referencer, FName AttributeName, int32 ReferencerChannelIndex)
{
	// Reset the built indices as the old ones will no longer be valid
	PerChannelIndices.Reset();
	bSuspended = false;

	// Cache a pointer to the element type container which is being referenced.
	// This may be multidimensional (e.g. in the case of a UV element type container, which is referenced by a UV attribute with multiple indices).
	ReferencedElementType = InKey;

	// Cache a reference to the attribute type which references that element container
	check(ReferencerChannelIndex >= 0 && ReferencerChannelIndex < Referencer->GetNumChannels());
	const FMeshElementContainer& Container = Referencer->Get(ReferencerChannelIndex);
	ReferencingAttributeRef = Container.GetAttributes().GetAttributesRef<TArrayView<int32>>(AttributeName);

	UpdatePerChannelIndices();
	bEverythingStale = true;
}


void FMeshElementIndexer::Reset()
{
	PerChannelIndices.Reset();
	bSuspended = false;

	// If either the referenced element type or the referencing attribute is invalid, clear out the built indices and return immediately
	if (ReferencedElementType == nullptr || !ReferencingAttributeRef.IsValid())
	{
		return;
	}

	UpdatePerChannelIndices();
	bEverythingStale = true;
}


void FMeshElementIndexer::ForceRebuild()
{
	bEverythingStale = true;
	bSuspended = false;
	Build();
}


void FMeshElementIndexer::RemoveKey(int32 KeyIndex, int32 KeyChannelIndex)
{
	if (bSuspended || bEverythingStale)
	{
		return;
	}

	if (KeyChannelIndex >= PerChannelIndices.Num())
	{
		PerChannelIndices.SetNum(KeyChannelIndex + 1);
	}

	check(KeyIndex != INDEX_NONE);
	int32 Chunk = GetChunkForKey(KeyIndex);
	int32 ChunkElement = GetChunkElementForKey(KeyIndex);

	if (Chunk < PerChannelIndices[KeyChannelIndex].Chunks.Num())
	{
		PerChannelIndices[KeyChannelIndex].Chunks[Chunk].Count[ChunkElement] = 0;
	}
	else
	{
		check(PerChannelIndices[KeyChannelIndex].StaleKeyIndices.Contains(KeyIndex));
	}
}


void FMeshElementIndexer::RemoveReferenceFromKey(int32 KeyIndex, int32 ReferenceIndex, int32 KeyChannelIndex)
{
	if (bSuspended || bEverythingStale)
	{
		return;
	}

	if (KeyChannelIndex >= PerChannelIndices.Num())
	{
		PerChannelIndices.SetNum(KeyChannelIndex + 1);
	}

	check(KeyIndex != INDEX_NONE);
	check(ReferenceIndex != INDEX_NONE);
	int32 Chunk = GetChunkForKey(KeyIndex);
	int32 ChunkElement = GetChunkElementForKey(KeyIndex);

	// Remove a reference immediately, as it is a quick operation.
	// However it's possible that the reference being removed has not yet even been lazily added.
	// If it doesn't exist, check that the key is at least marked as stale.
	if (Chunk < PerChannelIndices[KeyChannelIndex].Chunks.Num())
	{
		if (!PerChannelIndices[KeyChannelIndex].Chunks[Chunk].Remove(ChunkElement, ReferenceIndex))
		{
			check(PerChannelIndices[KeyChannelIndex].StaleKeyIndices.Contains(KeyIndex));
		}
	}
	else
	{
		check(PerChannelIndices[KeyChannelIndex].StaleKeyIndices.Contains(KeyIndex));
	}
}


void FMeshElementIndexer::AddReferenceToKey(int32 KeyIndex, int32 ReferenceIndex, int32 KeyChannelIndex)
{
	if (bSuspended || bEverythingStale)
	{
		return;
	}

	if (KeyChannelIndex >= PerChannelIndices.Num())
	{
		PerChannelIndices.SetNum(KeyChannelIndex + 1);
	}

	check(KeyIndex != INDEX_NONE);
	check(ReferenceIndex != INDEX_NONE);

	// Special handling for chunk sizes of 1, which can always be added directly
	if (ChunkBits == 0)
	{
		AddUnchunked(KeyIndex, ReferenceIndex, KeyChannelIndex);
		return;
	}

	int32 Chunk = GetChunkForKey(KeyIndex);

	bool bChunkAdded = false;

	// Try and add it to the key immediately if there's any room, unless the reference is already indexed
	if (Chunk < PerChannelIndices[KeyChannelIndex].Chunks.Num())
	{
		int32 ChunkElement = GetChunkElementForKey(KeyIndex);

		// Check if the key is not already in the chunk before attempting to add it.
		bChunkAdded = PerChannelIndices[KeyChannelIndex].Chunks[Chunk].Contains(ChunkElement, ReferenceIndex);

		if (!bChunkAdded)
		{
			bChunkAdded = PerChannelIndices[KeyChannelIndex].Chunks[Chunk].Add(ChunkElement, ReferenceIndex);
		}
	}
	
	// Otherwise mark key and referencer as stale and they will be built in batch at first need
	if (!bChunkAdded)
	{
		PerChannelIndices[KeyChannelIndex].StaleKeyIndices.Add(KeyIndex);
		PerChannelIndices[KeyChannelIndex].StaleReferencerIndices.Add(ReferenceIndex);
	}
}


void FMeshElementIndexer::ConditionalBuild(int32 KeyIndex, int32 KeyChannelIndex)
{
	// If we don't have an index for the requested element type index, just do a full build
	if (KeyChannelIndex >= PerChannelIndices.Num() || bEverythingStale)
	{
		Build();
	}
	// If the requested key is stale, build the index based on cached changes
	else if (PerChannelIndices[KeyChannelIndex].StaleKeyIndices.Contains(KeyIndex))
	{
		BuildIndex(KeyChannelIndex);
	}
}


void FMeshElementIndexer::UpdatePerChannelIndices()
{
	// We are able to handle attributes with multiple channels, corresponding to element containers with the same channels.
	//
	// For example:
	//
	//   VertexInstance Element Type
	//     - attribute UVIndex (2 channels)
	//
	//   refers to:
	//
	//   UV Element Type [2 channels]
	//     - attribute TextureCoordinate (FVector2f)
	//
	// If the number of channels doesn't correspond, we build the minimum number of channels shared by both referencer and referencee.

	int32 NumReferencingAttributeIndices = ReferencingAttributeRef.GetNumChannels();
	int32 NumElementTypeIndices = ReferencedElementType->GetNumChannels();
	int32 NumIndicesToUse = FMath::Min(NumReferencingAttributeIndices, NumElementTypeIndices);

	// If anything has changed, we will either remove surplus built indices, or add blank ones which are marked as requiring a full rebuild.
	PerChannelIndices.SetNum(NumIndicesToUse);
}


void FMeshElementIndexer::Build()
{
	check(!bSuspended);

	// If either the referenced element type or the referencing attribute is invalid, clear out the built indices and return immediately
	if (ReferencedElementType == nullptr || !ReferencingAttributeRef.IsValid())
	{
		PerChannelIndices.Reset();
		return;
	}

	UpdatePerChannelIndices();

	for (int32 Index = 0; Index < PerChannelIndices.Num(); Index++)
	{
		const FIndexPerChannel& PerChannelIndex = PerChannelIndices[Index];
		if (bEverythingStale)
		{
			RebuildIndex(Index);
		}
		else if (PerChannelIndex.StaleKeyIndices.Num() > 0)
		{
			BuildIndex(Index);
		}
	}

	bEverythingStale = false;
}


void FMeshElementIndexer::RebuildIndex(int32 Index)
{
	check(!bSuspended);

	// Check invariants
	check(Index >= 0);
	check(ReferencedElementType != nullptr);
	check(Index < ReferencedElementType->GetNumChannels());
	check(ReferencingAttributeRef.IsValid());
	check(Index < ReferencingAttributeRef.GetNumChannels());

	// Get referenced element container
	const FMeshElementContainer& ReferencedElementContainer = ReferencedElementType->Get(Index);
	int32 NumReferencedElements = ReferencedElementContainer.GetArraySize();
	int32 NumChunks = GetNumChunksForKeys(NumReferencedElements);

	// Initialize container for results
	FIndexPerChannel& PerChannelIndex = PerChannelIndices[Index];
	PerChannelIndex.Chunks.Reset();

	// Add all the chunks we need
	PerChannelIndex.Chunks.Reserve(NumChunks);
	while (PerChannelIndex.Chunks.Num() < NumChunks)
	{
		// Initially create the chunks with an initial size of 0.
		// They will be sized to at least the initial size in the following steps.
		PerChannelIndex.Chunks.Emplace(0, GetChunkSize());
	}

	// Get referencing attribute array
	int32 NumElements = ReferencingAttributeRef.GetNumElements();
	if (NumElements > 0 && NumReferencedElements > 0)
	{
		TArrayView<const int32> ReferencingAttributes = ReferencingAttributeRef.GetRawArray(Index);
		uint32 AttributeExtent = ReferencingAttributeRef.GetExtent();

		// First pass to determine number of references for each element
		// Iterate through the entire raw attribute array (number of elements * extent) counting references
		{
			for (int32 I = 0; I < ReferencingAttributes.Num(); I++)
			{
				// Read the index value of the attribute
				int32 Reference = ReferencingAttributes[I];
				if (Reference != INDEX_NONE)
				{
					// See which chunk that reference index lies in
					int32 Chunk = GetChunkForKey(Reference);
					int32 ChunkElement = GetChunkElementForKey(Reference);

					// Increment the count for that referenced index: like this we can determine how many references each will have
					++PerChannelIndex.Chunks[Chunk].MaxCount[ChunkElement];
				}
			}
		}

		// Second pass to determine start indices
		{
			for (FChunk& Chunk : PerChannelIndex.Chunks)
			{
				int32 CurrentStartIndex = 0;
				const int32 ChunkSize = GetChunkSize();
				for (int32 I = 0; I < ChunkSize; I++)
				{
					// For each chunk, iterate through the counts accumulated for each element, and accumulate start indices.
					// We force the max count for each element to be at least InitialSize.
					Chunk.StartIndex[I] = CurrentStartIndex;
					Chunk.MaxCount[I] = FMath::Max(InitialSize, Chunk.MaxCount[I]);
					CurrentStartIndex += Chunk.MaxCount[I];
				}

				// Preallocate data buffer for each chunk
				Chunk.Data.SetNumUninitialized(CurrentStartIndex, EAllowShrinking::No);
			}
		}

		// Third pass to fill in data
		// Iterate through the referencing array, adding the array index to the key 
		{
			int32 I = 0;
			for (int32 ElementIndex = 0; ElementIndex < NumElements; ElementIndex++)
			{
				for (uint32 N = 0; N < AttributeExtent; N++)
				{
					int32 Reference = ReferencingAttributes[I];
					if (Reference != INDEX_NONE)
					{
						// See which chunk that reference index lies in
						int32 Chunk = GetChunkForKey(Reference);
						int32 ChunkElement = GetChunkElementForKey(Reference);

						// Add the value, but do not add it in its sorted position for speed
						PerChannelIndex.Chunks[Chunk].AddUnsorted(ChunkElement, ElementIndex);
					}

					I++;
				}
			}
		}

		// Final pass to sort the data
		{
			for (FChunk& Chunk : PerChannelIndex.Chunks)
			{
				const int32 ChunkSize = GetChunkSize();
				for (int32 I = 0; I < ChunkSize; I++)
				{
					Chunk.Sort(I);
				}
			}
		}
	}

	PerChannelIndex.StaleKeyIndices.Reset();
	PerChannelIndex.StaleReferencerIndices.Reset();
}


void FMeshElementIndexer::BuildIndex(int32 Index)
{
	// The aim of this method is to insert new blocks into the middle of the data array as required, performing a single alloc grow,
	// and only moving data once.
	// We build a list of the keys which are to be updated, and how much extra space each needs.
	// Once we know, add elements to the end of the Data array, and then, working backwards, shift blocks to the end of the Data array,
	// leaving spaces where required for new references at each updated key index.

	check(!bSuspended);

	FIndexPerChannel& PerChannelIndex = PerChannelIndices[Index];

	// A list of all the keys which will be updated
	TArray<int32, TInlineAllocator<256>> KeysToUpdate;

	// Ensure we have enough chunks to accommodate the current number of keys.
	// Any new ones will be default initialized to appropriate values.
	const FMeshElementContainer& ReferencedElementContainer = ReferencedElementType->Get(Index);
	int32 NumReferencedElements = ReferencedElementContainer.GetArraySize();
	int32 NumChunks = GetNumChunksForKeys(NumReferencedElements);

	// Add extra chunks if necessary
	PerChannelIndex.Chunks.Reserve(NumChunks);
	while (PerChannelIndex.Chunks.Num() < NumChunks)
	{
		PerChannelIndex.Chunks.Emplace(InitialSize, GetChunkSize());
	}

	// Remove extra chunks if too many
	if (PerChannelIndex.Chunks.Num() > NumChunks)
	{
		PerChannelIndex.Chunks.RemoveAt(NumChunks, PerChannelIndex.Chunks.Num() - NumChunks, EAllowShrinking::No);
	}

	// First pass to determine the keys which need updating
	{
		for (int32 StaleReferencerIndex : PerChannelIndex.StaleReferencerIndices)
		{
			TArrayView<const int32> References = ReferencingAttributeRef.Get(StaleReferencerIndex, Index);
			for (int32 Reference : References)
			{
				if (Reference != INDEX_NONE)
				{
					int32 Chunk = GetChunkForKey(Reference);
					int32 ChunkElement = GetChunkElementForKey(Reference);
					if (!PerChannelIndex.Chunks[Chunk].Get(ChunkElement).Contains(StaleReferencerIndex))
					{
						KeysToUpdate.Add(Reference);
					}
				}
			}
		}
	}

	// Second pass to determine how much extra memory to allocate for each chunk.
	// First sort the key indices, so we can process them chunk at a time.
	Algo::Sort(KeysToUpdate);

	// Fix up each chunk
	int32 I = 0;
	while (I < KeysToUpdate.Num())
	{
		// Get the index bounds for keys in a single chunk
		int32 StartI = I;
		int32 CurrentChunk = GetChunkForKey(KeysToUpdate[I]);
		int32 ExtraChunkElements = 0;
		while (++I < KeysToUpdate.Num())
		{
			if (GetChunkForKey(KeysToUpdate[I]) != CurrentChunk)
			{
				break;
			}
		}
		// This chunk now referenced by KeysToUpdate [StartI, I).

		FChunk& ThisChunk = PerChannelIndex.Chunks[CurrentChunk];

		int32 J = StartI;
		while (J < I)
		{
			// Search for contiguous runs of the same key index
			int32 KeyToUpdate = KeysToUpdate[J];
			int32 ContiguousEqualKeys = 1;
			while (++J < I)
			{
				if (KeysToUpdate[J] == KeyToUpdate)
				{
					ContiguousEqualKeys++;
				}
				else
				{
					break;
				}
			}

			int32 ChunkElement = GetChunkElementForKey(KeyToUpdate);
			int32 MaxCount = ThisChunk.MaxCount[ChunkElement];
			int32 Count = ThisChunk.Count[ChunkElement];
			ExtraChunkElements += FMath::Max(0, ContiguousEqualKeys - (MaxCount - Count));
		}

		// At this point, we know how many extra elements are required in this chunk to accommodate new references.
		// Now we need to add them, and make space in the data array for the extra items in the right place.

		// Add the extra memory and insert the new elements
		ThisChunk.Data.AddDefaulted(ExtraChunkElements);

		// LastDataElement is the end index we will move the block to.
		int32 LastDataElement = ThisChunk.Data.Num();

		int32 LastKeyToMove = GetChunkSize();

		int32* DataPtr = ThisChunk.Data.GetData();
	
		int32 K = I - 1;
		while (K >= StartI)
		{
			// Determine how many equal keys are contiguous; this determines how much space we need to leave.
			int32 KeyToUpdate = KeysToUpdate[K];
			int32 ContiguousEqualKeys = 1;
			while (--K >= StartI)
			{
				if (KeysToUpdate[K] == KeyToUpdate)
				{
					ContiguousEqualKeys++;
				}
				else
				{
					break;
				}
			}

			// We do not move the index which is being added to; we move from the next one to the end of the newly available space
			// FirstDataElement is the index we are moving from.
			int32 ChunkElement = GetChunkElementForKey(KeyToUpdate);
			int32 FirstDataElement = ThisChunk.StartIndex[ChunkElement] + ThisChunk.MaxCount[ChunkElement];
			int32 ElementsAdded = FMath::Max(0, ContiguousEqualKeys - (ThisChunk.MaxCount[ChunkElement] - ThisChunk.Count[ChunkElement]));

			FMemory::Memmove(DataPtr + FirstDataElement + ExtraChunkElements, DataPtr + FirstDataElement, sizeof(int32) * (LastDataElement - FirstDataElement - ExtraChunkElements));

			// Now extend the max count of the index we have just made more space for.
			ThisChunk.MaxCount[ChunkElement] += ElementsAdded;

			// And adjust the start positions in the data block of subsequent key indices
			for (int N = GetChunkElementForKey(KeyToUpdate) + 1; N < LastKeyToMove; N++)
			{
				check(N < GetChunkSize());
				ThisChunk.StartIndex[N] += ExtraChunkElements;
			}

			// The LastDataElement now becomes the start of the block we just moved, in its new position.
			// Anything beyond there is already in its final place.
			LastDataElement = FirstDataElement + ExtraChunkElements;
			LastKeyToMove = GetChunkElementForKey(KeyToUpdate) + 1;

			// And decrease the ExtraChunkElements count; lower indices have to move less.
			ExtraChunkElements -= ElementsAdded;
			check(ExtraChunkElements >= 0);
		}
	}

	// Third pass: populate new references into the keys
	{
		for (int32 StaleReferencerIndex : PerChannelIndex.StaleReferencerIndices)
		{
			TArrayView<const int32> References = ReferencingAttributeRef.Get(StaleReferencerIndex, Index);
			for (int32 Reference : References)
			{
				if (Reference != INDEX_NONE)
				{
					int32 Chunk = GetChunkForKey(Reference);
					int32 ChunkElement = GetChunkElementForKey(Reference);
					if (!PerChannelIndex.Chunks[Chunk].Get(ChunkElement).Contains(StaleReferencerIndex))
					{
						PerChannelIndex.Chunks[Chunk].Add(ChunkElement, StaleReferencerIndex);
					}
				}
			}
		}
	}

	PerChannelIndex.StaleKeyIndices.Reset();
	PerChannelIndex.StaleReferencerIndices.Reset();
}


void FMeshElementIndexer::AddUnchunked(int32 KeyIndex, int32 ReferenceIndex, int32 KeyChannelIndex)
{
	check(ChunkBits == 0);

	FIndexPerChannel& PerChannelIndex = PerChannelIndices[KeyChannelIndex];

	// Ensure we have enough chunks to accommodate the current number of keys.
	// Any new ones will be default initialized to appropriate values.
	const FMeshElementContainer& ReferencedElementContainer = ReferencedElementType->Get(KeyChannelIndex);
	int32 NumReferencedElements = ReferencedElementContainer.GetArraySize();

	// Add extra chunks if necessary
	PerChannelIndex.Chunks.Reserve(NumReferencedElements);
	while (PerChannelIndex.Chunks.Num() < NumReferencedElements)
	{
		PerChannelIndex.Chunks.Emplace(InitialSize, 1);
	}

	// Remove extra chunks if too many
	if (PerChannelIndex.Chunks.Num() > NumReferencedElements)
	{
		PerChannelIndex.Chunks.RemoveAt(NumReferencedElements, PerChannelIndex.Chunks.Num() - NumReferencedElements, EAllowShrinking::No);
	}

	FChunk& Chunk = PerChannelIndex.Chunks[KeyIndex];
	check(Chunk.StartIndex[0] == 0);
	if (Chunk.Count[0] == Chunk.MaxCount[0])
	{
		// If the chunk is full, expand it by 50%
		check(Chunk.Data.Num() == Chunk.MaxCount[0]);
		Chunk.MaxCount[0] += (Chunk.MaxCount[0] / 2);
		Chunk.Data.SetNumUninitialized(Chunk.MaxCount[0]);
	}

	Chunk.Add(0, ReferenceIndex);
}


bool FMeshElementIndexer::FChunk::Add(int32 Index, int32 Value)
{
	if (Count[Index] >= MaxCount[Index])
	{
		return false;
	}

	// Build array view for the portion of the data array corresponding to this element
	int32* Ptr = Data.GetData() + StartIndex[Index];
	TArrayView<int32> View(Ptr, Count[Index]);

	// Find index in sorted view where this value would go
	int32 ViewIndex = Algo::LowerBound(View, Value);
	bool bExists = (ViewIndex < Count[Index] && View[ViewIndex] == Value);
	ensureMsgf(!bExists, TEXT("FMeshElementIndexer: duplicate reference %d added to key (chunk element %d)"), Value, Index);

	if (!bExists)
	{
		FMemory::Memmove(Ptr + ViewIndex + 1, Ptr + ViewIndex, sizeof(int32) * (Count[Index] - ViewIndex));
		Ptr[ViewIndex] = Value;
		Count[Index]++;
	}

	return true;
}


bool FMeshElementIndexer::FChunk::AddUnsorted(int32 Index, int32 Value)
{
	// If a duplicate entry gets into the list, this will fire
	check(Count[Index] < MaxCount[Index]);

	int32* Ptr = Data.GetData() + StartIndex[Index];
	Ptr[Count[Index]] = Value;
	Count[Index]++;

	return true;
}


void FMeshElementIndexer::FChunk::Sort(int32 Index)
{
	int32* Ptr = Data.GetData() + StartIndex[Index];
	TArrayView<int32> View(Ptr, Count[Index]);
	Algo::Sort(View);
}


bool FMeshElementIndexer::FChunk::Remove(int32 Index, int32 Value)
{
	// Build array view for the portion of the data array corresponding to this element
	int32* Ptr = Data.GetData() + StartIndex[Index];
	TArrayView<int32> View(Ptr, Count[Index]);

	// Search for the value in the view
	int32 ViewIndex = Algo::LowerBound(View, Value);

	if (ViewIndex < Count[Index] && View[ViewIndex] == Value)
	{
		FMemory::Memmove(Ptr + ViewIndex, Ptr + ViewIndex + 1, sizeof(int32) * (Count[Index] - ViewIndex - 1));
		Count[Index]--;
		return true;
	}

	return false;
}


bool FMeshElementIndexer::FChunk::Contains(int32 Index, int32 Value) const
{
	// Build array view for the portion of the data array corresponding to this element
	const int32* Ptr = Data.GetData() + StartIndex[Index];
	TArrayView<const int32> View(Ptr, Count[Index]);

	// Find index in sorted view where this value would go
	int32 ViewIndex = Algo::LowerBound(View, Value);
	return (ViewIndex < Count[Index] && View[ViewIndex] == Value);
}

