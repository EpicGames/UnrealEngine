// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshTypes)

DEFINE_LOG_CATEGORY( LogMeshDescription );


const FElementID FElementID::Invalid( INDEX_NONE );
const FVertexID FVertexID::Invalid( INDEX_NONE );
const FVertexInstanceID FVertexInstanceID::Invalid( INDEX_NONE );
const FEdgeID FEdgeID::Invalid( INDEX_NONE );
const FTriangleID FTriangleID::Invalid( INDEX_NONE );
const FPolygonGroupID FPolygonGroupID::Invalid( INDEX_NONE );
const FPolygonID FPolygonID::Invalid( INDEX_NONE );

