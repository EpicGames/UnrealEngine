// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTypes.h"
#include "MeshElementArray.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

constexpr const EAutomationTestFlags TestFlags = EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter;

/**
 * A very basic type that can be used as the ElementIDType with TMeshElementArray in our tests
 */
namespace
{
	struct FTestID : public FElementID
	{
		FTestID()
		{
		}

		FTestID(const FElementID InitElementID)
			: FElementID(InitElementID.GetValue())
		{
		}

		FTestID(const int32 InitIDValue)
			: FElementID(InitIDValue)
		{
		}
	};
}

/**
 * Test basic serialization of TMeshElementArray
 * Create a TMeshElementArray with many entries and then remove a number of them so that we have slack in the container (as well as a number of
 * invalid elements at the end of the array) and then serialize it to a byte array and then back again into a new TMeshElementArray container.
 * We can then verify that the byte arrays and the containers are the same.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMeshElementArrayTestSerialization, TEXT("System.MeshDescription.MeshElementArray.Serialization"), TestFlags)
bool FMeshElementArrayTestSerialization::RunTest(const FString& Parameters)
{
	TMeshElementArray<int32, FTestID> OriginalArray;
	
	const int32 NumElementsToAdd = 10 * 1024;				// The number of elements to initially add
	const int32 RemovalFrequency = 3;						// Remove every third element
	const int32 CutOffPoint = (NumElementsToAdd / 4) * 3;	// We will remove the last quarter of elements

	TArray<FTestID> ElementsToRemove;
	for (int32 Index = 0; Index < NumElementsToAdd; ++Index)
	{
		FTestID ElementID = OriginalArray.Add();
		OriginalArray[ElementID] = Index;

		// Check if we want to remove this element and if so add it to the list
		if (Index % RemovalFrequency || Index >= CutOffPoint)
		{
			ElementsToRemove.Add(ElementID);
		}
	}

	// Now remove the elements to create slack in the array
	for (FTestID ElementID : ElementsToRemove)
	{
		OriginalArray.Remove(ElementID);
	}
	
	TArray<uint8> OriginalArrayBytes;

	// Serialize the original array to a byte array
	{
		FMemoryWriter Ar(OriginalArrayBytes);
		Ar << OriginalArray;
	}

	TMeshElementArray<int32, FTestID> NewArray;

	// Serialize the byte array to a new array
	{
		FMemoryReader Ar(OriginalArrayBytes);
		Ar << NewArray;
	}

	// Lastly serialize the new array back to a byte array for easy comparison
	TArray<uint8> NewArrayBytes;

	{
		FMemoryWriter Ar(NewArrayBytes);
		Ar << NewArray;
	}

	// Test that the byte arrays are the same
	TestTrue(TEXT("Byte arrays are the same length"), OriginalArrayBytes.Num() == NewArrayBytes.Num());
	TestTrue(TEXT("Byte arrays are binary equivalent"), FMemory::Memcmp(OriginalArrayBytes.GetData(), NewArrayBytes.GetData(), OriginalArrayBytes.Num()) == 0);

	// Test that the TMeshElementArray are the same
	TestTrue(TEXT("MeshElementArrays are the same number of valid elements"), OriginalArray.Num() == NewArray.Num());

	const int32 MaxElement = FMath::Min(OriginalArray.GetArraySize(), NewArray.GetArraySize());
	for (int32 Index = 0; Index < NumElementsToAdd; ++Index)
	{
		TestTrue(TEXT("MeshElementArrays have the same validity"), OriginalArray.IsValid(Index) == NewArray.IsValid(Index));
		if (OriginalArray.IsValid(Index) && NewArray.IsValid(Index))
		{
			TestTrue(TEXT("MeshElementArrays have the same element value"), OriginalArray[Index] == NewArray[Index]);
		}
	}

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
