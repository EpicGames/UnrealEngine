// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/StaticArray.h"
#include "Algo/Rotate.h"


template <typename AttributeType>
class TAttributeArrayContainer
{
public:
	TAttributeArrayContainer() = default;

	explicit TAttributeArrayContainer(const AttributeType& InDefaultValue)
		: DefaultValue(InDefaultValue)
	{}

	/** Return size of container */
	int32 Num() const { return NumElements; }

	/** Initializes the array to the given size with the default value */
	void Initialize(const int32 ElementCount, const AttributeType& Default)
	{
		// For unbounded arrays, the default value is an empty subarray.
		check(ElementCount >= 0);
		int32 NumChunks = GetNumRequiredChunks(ElementCount);
		Chunks.Empty(NumChunks);
		Chunks.SetNum(NumChunks);
		if (NumChunks > 0)
		{
			// Set num elements in the final chunk to the right value.
			// All previous chunks are default initialized to be 'full'.
			Chunks.Last().NumElements = GetElementsInLastChunk(ElementCount);
		}

		NumElements = ElementCount;
	}

	/** Sets the number of elements, each element itself being a subarray of items of type AttributeType. */
	void SetNum(const int32 ElementCount, const AttributeType& Default)
	{
		check(ElementCount >= 0);

		int32 OldNumChunks = Chunks.Num();
		int32 NumChunks = GetNumRequiredChunks(ElementCount);
		Chunks.SetNum(NumChunks);

		if (ElementCount < NumElements)
		{
			// Case where we're shrinking the unbounded array
			if (NumChunks > 0)
			{
				Chunks.Last().NumElements = GetElementsInLastChunk(ElementCount);
			}
		}
		else if (ElementCount > NumElements)
		{
			// Case where we're growing the unbounded array
			int32 IndexInOldLastChunk = NumElements & ChunkMask;
			int32 ElementsInLastChunk = GetElementsInLastChunk(ElementCount);

			if (IndexInOldLastChunk > 0)
			{
				// If the original final chunk had unused elements, we need to initialize their data pointers up to either:
				// - the new final element, if it is still the final chunk
				// - the end (ChunkSize) if it is no longer the final chunk
				int32 LastIndex = (NumChunks == OldNumChunks) ? ElementsInLastChunk : ChunkSize;

				FChunk& OldLastChunk = Chunks[OldNumChunks - 1];

				// Determine the start index in the data block of the next element,
				// and resize it to that amount
				int32 DataStartIndex = OldLastChunk.StartIndex[IndexInOldLastChunk - 1] + OldLastChunk.MaxCount[IndexInOldLastChunk - 1];
				OldLastChunk.Data.SetNum(DataStartIndex, EAllowShrinking::No);

				for (; IndexInOldLastChunk < LastIndex; IndexInOldLastChunk++)
				{
					OldLastChunk.StartIndex[IndexInOldLastChunk] = DataStartIndex;
					OldLastChunk.Count[IndexInOldLastChunk] = 0;
					OldLastChunk.MaxCount[IndexInOldLastChunk] = 0;
				}

				// Update the element count for the old last chunk
				OldLastChunk.NumElements = LastIndex;
			}

			// Any further chunks which have just been initialized will default to being 'full', i.e. with ChunkSize elements
			// The very last chunk will need initializing to the correct value.
			// If this happens to coincide with the old last chunk, it doesn't hurt to do this again here.
			Chunks.Last().NumElements = ElementsInLastChunk;
		}

		NumElements = ElementCount;
	}

	uint32 GetHash(uint32 Crc = 0) const
	{
		for (const FChunk& Chunk : Chunks)
		{
			Crc = FCrc::MemCrc32(Chunk.Data.GetData(), Chunk.Data.Num() * sizeof(AttributeType), Crc);
		}
		return Crc;
	}

	/** Expands the array if necessary so that the passed element index is valid. Newly created elements will be assigned the default value. */
	void Insert(const int32 Index, const AttributeType& Default)
	{
		check(Index >= 0);

		int32 EndIndex = Index + 1;
		if (EndIndex > NumElements)
		{
			SetNum(EndIndex, Default);
		}
	}

	/** Fills the index with the default value */
	void SetToDefault(const int32 Index, const AttributeType& Default)
	{
		checkSlow(Index >= 0 && Index < NumElements);
		int32 ChunkIndex = Index >> ChunkBits;
		int32 ElementIndex = Index & ChunkMask;

		// Default value is an empty array. We do not compact the allocations when shrinking the data.
		Chunks[ChunkIndex].Count[ElementIndex] = 0;
	}

	/** Remaps elements according to the passed remapping table */
	void Remap(const TSparseArray<int32>& IndexRemap, const AttributeType& Default);

	friend FArchive& operator <<(FArchive& Ar, TAttributeArrayContainer<AttributeType>& Array)
	{
		Ar << Array.Chunks;
		Ar << Array.NumElements;
		Ar << Array.DefaultValue;
		return Ar;
	}

	/** Gets the array attribute at the given index as a TArrayView */
	TArrayView<AttributeType> Get(int32 Index)
	{
		checkSlow(Index >= 0 && Index < NumElements);
		int32 ChunkIndex = Index >> ChunkBits;
		int32 ElementIndex = Index & ChunkMask;
		FChunk& Chunk = Chunks[ChunkIndex];
		return TArrayView<AttributeType>(Chunk.Data.GetData() + Chunk.StartIndex[ElementIndex], Chunk.Count[ElementIndex]);
	}

	/** Gets the array attribute at the given index as a TArrayView */
	TArrayView<const AttributeType> Get(int32 Index) const
	{
		checkSlow(Index >= 0 && Index < NumElements);
		int32 ChunkIndex = Index >> ChunkBits;
		int32 ElementIndex = Index & ChunkMask;
		const FChunk& Chunk = Chunks[ChunkIndex];
		return TArrayView<const AttributeType>(Chunk.Data.GetData() + Chunk.StartIndex[ElementIndex], Chunk.Count[ElementIndex]);
	}

	/** Gets the element count of the array attribute at the given index */
	int32 GetElementCount(int32 Index) const
	{
		checkSlow(Index >= 0 && Index < NumElements);
		int32 ChunkIndex = Index >> ChunkBits;
		int32 ElementIndex = Index & ChunkMask;
		const FChunk& Chunk = Chunks[ChunkIndex];
		return Chunk.Count[ElementIndex];
	}

	/** Gets the reserved element count of the array attribute at the given index */
	int32 GetReservedElementCount(int32 Index) const
	{
		checkSlow(Index >= 0 && Index < NumElements);
		int32 ChunkIndex = Index >> ChunkBits;
		int32 ElementIndex = Index & ChunkMask;
		const FChunk& Chunk = Chunks[ChunkIndex];
		return Chunk.MaxCount[ElementIndex];
	}

	/** Sets the attribute array at the given index to the given TArrayView */
	void Set(int32 Index, TArrayView<const AttributeType> Value)
	{
		const bool bSetCount = true;
		const bool bSetDefault = false;
		TArrayView<AttributeType> Element = SetElementSize(Index, Value.Num(), bSetCount, bSetDefault);

		for (int32 I = 0; I < Value.Num(); I++)
		{
			Element[I] = Value[I];
		}
	}

	/** Sets the given attribute array element to have the given number of subarray elements. */
	TArrayView<AttributeType> SetElementSize(int32 Index, int32 Size, bool bSetCount, bool bSetDefault = true)
	{
		checkSlow(Index >= 0 && Index < NumElements);
		int32 ChunkIndex = Index >> ChunkBits;
		int32 ElementIndex = Index & ChunkMask;
		FChunk& Chunk = Chunks[ChunkIndex];

		AttributeType* BasePtr = Chunk.Data.GetData();

		int32 Extra = Size - Chunk.MaxCount[ElementIndex];
		if (Extra > 0)
		{
			// If we are requesting that MaxCount grows for this element, we have to insert new data elements,
			// and adjust subsequent start indices accordingly.
			int32 OldDataSize = Chunk.Data.Num();
			Chunk.Data.SetNum(OldDataSize + Extra);
			BasePtr = Chunk.Data.GetData();

			AttributeType* StartPtr = BasePtr + Chunk.StartIndex[ElementIndex] + Chunk.MaxCount[ElementIndex];
			AttributeType* EndPtr = BasePtr + OldDataSize;
			FMemory::Memmove(StartPtr + Extra, StartPtr, (EndPtr - StartPtr) * sizeof(AttributeType));

			Chunk.MaxCount[ElementIndex] = Size;
			for (int32 I = ElementIndex + 1; I < Chunk.NumElements; I++)
			{
				Chunk.StartIndex[I] += Extra;
			}
		}

		if (bSetDefault)
		{
			if (Size > Chunk.Count[ElementIndex])
			{
				// Set any newly created members to the default value
				for (int32 I = Chunk.Count[ElementIndex]; I < Chunk.MaxCount[ElementIndex]; I++)
				{
					Chunk.Data[Chunk.StartIndex[ElementIndex] + I] = DefaultValue;
				}
			}
			else
			{
				// Set any removed values to the default value
				for (int32 I = Size; I < Chunk.Count[ElementIndex]; I++)
				{
					Chunk.Data[Chunk.StartIndex[ElementIndex] + I] = DefaultValue;
				}
			}
		}

		if (bSetCount)
		{
			Chunk.Count[ElementIndex] = Size;
		}

		return TArrayView<AttributeType>(BasePtr + Chunk.StartIndex[ElementIndex], Chunk.Count[ElementIndex]);
	}

	TArrayView<AttributeType> InsertIntoElement(int32 Index, int32 SubArrayIndex, int32 InsertCount = 1)
	{
		checkSlow(Index >= 0 && Index < NumElements);
		int32 ChunkIndex = Index >> ChunkBits;
		int32 ElementIndex = Index & ChunkMask;
		FChunk& Chunk = Chunks[ChunkIndex];

		int32 CurrentCount = Chunk.Count[ElementIndex];
		checkSlow(SubArrayIndex >= 0 && SubArrayIndex <= CurrentCount);
		int32 NewCount = CurrentCount + InsertCount;

		if (NewCount > Chunk.MaxCount[ElementIndex])
		{
			// Add a few extra slots in anticipation of further growth in the future
			// TODO: opt in to this behavior?
			NewCount = NewCount * 4 / 3;

			// Resize the allocated space for this element if necessary
			const bool bSetCount = false;
			const bool bSetDefault = true;
			SetElementSize(Index, NewCount, bSetCount, bSetDefault);
		}

		Chunk.Count[ElementIndex] += InsertCount;

		// Insert the new subarray elements
		TArrayView<AttributeType> Element = Get(Index);
		int32 SpanSize = NewCount - SubArrayIndex;
		TArrayView<AttributeType> ElementToRotate(Element.GetData() + SubArrayIndex, SpanSize);
		Algo::Rotate(ElementToRotate, SpanSize - InsertCount);

		return Element;
	}

	TArrayView<AttributeType> RemoveFromElement(int32 Index, int32 SubArrayIndex, int32 Count = 1)
	{
		TArrayView<AttributeType> Element = Get(Index);
		for (int32 I = SubArrayIndex + Count; I < Element.Num(); I++)
		{
			Element[I - Count] = Element[I];
		}

		const bool bSetCount = true;
		const bool bSetDefault = true;
		return SetElementSize(Index, Element.Num() - Count, bSetCount, bSetDefault);
	}

	AttributeType GetDefault() const { return DefaultValue; }

private:

	static int32 GetNumRequiredChunks(const int32 ElementCount)
	{
		return (ElementCount + ChunkSize - 1) >> ChunkBits;
	}

	static int32 GetElementsInLastChunk(const int32 ElementCount)
	{
		return ((ElementCount + ChunkSize - 1) & ChunkMask) + 1;
	}

	static const int32 ChunkBits = 8;
	static const int32 ChunkSize = (1 << ChunkBits);
	static const int32 ChunkMask = ChunkSize - 1;

	struct FChunk
	{
		FChunk()
		{
			Data.Reserve(ChunkSize);
		}

		friend FArchive& operator <<(FArchive& Ar, FChunk& Chunk)
		{
			Ar << Chunk.Data;
			Ar << Chunk.NumElements;

			for (int32 Index = 0; Index < Chunk.NumElements; Index++)
			{
				Ar << Chunk.StartIndex[Index];
			}

			for (int32 Index = 0; Index < Chunk.NumElements; Index++)
			{
				Ar << Chunk.Count[Index];
			}

			for (int32 Index = 0; Index < Chunk.NumElements; Index++)
			{
				Ar << Chunk.MaxCount[Index];
			}

			return Ar;
		}

		// All the data for each element in the chunk, packed contiguously
		TArray<AttributeType> Data;

		// Start, count and allocated count in the Data array for each element in the chunk.
		// Arranged as SoA for cache optimization, since the most frequent operation is
		// adding a fixed amount to all the start indices when a value is inserted.
		TStaticArray<int32, ChunkSize> StartIndex{InPlace, 0};
		TStaticArray<int32, ChunkSize> Count     {InPlace, 0};
		TStaticArray<int32, ChunkSize> MaxCount  {InPlace, 0};

		// Specify number of valid elements in this chunk.
		// Default to maximum number so that we can add chunks in bulk, and all put the final
		// chunk will already be correctly initialized.
		int32 NumElements = ChunkSize;
	};

	TArray<FChunk> Chunks;
	int32 NumElements = 0;
	AttributeType DefaultValue = AttributeType();
};



template <typename AttributeType>
void TAttributeArrayContainer<AttributeType>::Remap(const TSparseArray<int32>& IndexRemap, const AttributeType& Default)
{
	TAttributeArrayContainer<AttributeType> NewAttributeArray(Default);

	for (TSparseArray<int32>::TConstIterator It(IndexRemap); It; ++It)
	{
		const int32 OldElementIndex = It.GetIndex();
		const int32 NewElementIndex = IndexRemap[OldElementIndex];

		NewAttributeArray.Insert(NewElementIndex, Default);
		NewAttributeArray.Set(NewElementIndex, Get(OldElementIndex));
	}

	*this = MoveTemp(NewAttributeArray);
}


/**
 * Proxy object which fields access to an unbounded array attribute container.
 */
template <typename AttributeType>
class TArrayAttribute
{
	template <typename T> friend class TArrayAttribute;
	using ArrayType = TCopyQualifiersFromTo_T<AttributeType, TAttributeArrayContainer<std::remove_cv_t<AttributeType>>>;

public:
	explicit TArrayAttribute(ArrayType& InArray, int32 InIndex)
		: Array(&InArray),
		  Index(InIndex)
	{}

	/**
	 * Construct a TArrayAttribute<const T> from a TArrayAttribute<T>. 
	 */
	template <typename T = AttributeType, typename TEnableIf<std::is_same_v<T, const T>, int>::Type = 0>
	TArrayAttribute(TArrayAttribute<std::remove_cv_t<T>> InValue)
		: Array(InValue.Array),
		  Index(InValue.Index)
	{}

	/**
	 * Helper function for returning a typed pointer to the first array attribute entry.
	 */
	AttributeType* GetData() const
	{
		return Array->Get(Index).GetData();
	}

	/**
	 * Tests if index is valid, i.e. than or equal to zero, and less than the number of elements in the array attribute.
	 *
	 * @param Index Index to test.
	 *
	 * @returns True if index is valid. False otherwise.
	 */
	bool IsValidIndex(int32 ArrayAttributeIndex) const
	{
		return Array->Get(Index).IsValidIndex(ArrayAttributeIndex);
	}

	/**
	 * Returns true if the array attribute is empty and contains no elements. 
	 *
	 * @returns True if the array attribute is empty.
	 */
	bool IsEmpty() const
	{
		return Array->Get(Index).IsEmpty();
	}

	/**
	 * Returns number of elements in the array attribute.
	 *
	 * @returns Number of elements in array attribute.
	 */
	int32 Num() const
	{
		return Array->Get(Index).Num();
	}

	/**
	 * Array bracket operator. Returns reference to array attribute element at given index.
	 *
	 * @returns Reference to indexed element.
	 */
	AttributeType& operator[](int32 ArrayAttributeIndex) const
	{
		return Array->Get(Index)[ArrayAttributeIndex];
	}

	/**
	 * Returns n-th last element from the array attribute.
	 *
	 * @param IndexFromTheEnd (Optional) Index from the end of array.
	 *                        Default is 0.
	 *
	 * @returns Reference to n-th last element from the array.
	 */
	AttributeType& Last(int32 IndexFromTheEnd = 0) const
	{
		return Array->Get(Index).Last();
	}

	/**
	 * Sets the number of elements in the array attribute.
	 */
	void SetNum(int32 Num)
	{
		const bool bSetCount = true;
		const bool bSetDefault = true;
		Array->SetElementSize(Index, Num, bSetCount, bSetDefault);
	}


	/**
	 * Sets the number of elements in the array attribute without initializing
	 * any new elements. It is the responsibility of the caller to do so after this 
	 * function returns.
	 */
	void SetNumUninitialized(int32 Num)
	{
		const bool bSetCount = true;
		const bool bSetDefault = false;
		Array->SetElementSize(Index, Num, bSetCount, bSetDefault);
	}


	/**
	 * Reserves the number of elements in the array attribute.
	 */
	void Reserve(int32 Num)
	{
		const bool bSetCount = false;
		const bool bSetDefault = true;
		Array->SetElementSize(Index, Num, bSetCount, bSetDefault);
	}


	/**
	 * Adds the given value to the end of the array attribute, reserving extra space if necessary.
	 */
	int32 Add(const AttributeType& Value)
	{
		int32 ElementCount = Array->GetElementCount(Index);
		TArrayView<AttributeType> Element = Array->InsertIntoElement(Index, ElementCount, 1);
		Element[ElementCount] = Value;
		return ElementCount;
	}


	/**
	 * Adds the given value to the end of the array attribute, reserving extra space if necessary.
	 */
	int32 AddUnique(const AttributeType& Value)
	{
		int32 FoundIndex = Find(Value);
		if (FoundIndex == INDEX_NONE)
		{
			return Add(Value);
		}
		else
		{
			return FoundIndex;
		}
	}


	/**
	 * Inserts a number of default-constructed elements in the array attribute
	 */
	void InsertDefaulted(int32 StartIndex, int32 Count)
	{
		Array->InsertIntoElement(Index, StartIndex, Count);
	}


	/**
	 * Inserts an element into the given index in the array attribute
	 */
	void Insert(const AttributeType& Value, int32 StartIndex)
	{
		TArrayView<AttributeType> Element = Array->InsertIntoElement(Index, StartIndex, 1);
		Element[StartIndex] = Value;
	}


	/**
	 * Removes a number of elements from the array attribute
	 */
	void RemoveAt(int32 StartIndex, int32 Count)
	{
		Array->RemoveFromElement(Index, StartIndex, Count);
	}


	/**
	 * Removes all elements which match the predicate
	 */
	template <typename Predicate>
	int32 RemoveAll(Predicate Pred)
	{
		TArrayView<AttributeType> Element = Array->Get(Index);
		int32 Count = Element.Num();

		int32 WriteIndex = 0;
		for (int32 ReadIndex = 0; ReadIndex < Count; ReadIndex++)
		{
			if (!Pred(Element[ReadIndex]))
			{
				Element[WriteIndex] = Element[ReadIndex];
				WriteIndex++;
			}
		}

		const bool bSetCount = true;
		const bool bSetDefault = true;
		Array->SetElementSize(Index, WriteIndex, bSetCount, bSetDefault);

		return Count - WriteIndex;
	}


	int32 Remove(const AttributeType& Value)
	{
		return RemoveAll([&Value](const AttributeType& ToCompare) { return Value == ToCompare; });
	}


	/**
	 * Finds the given value in the array attribute and returns its index
	 */
	int32 Find(const AttributeType& Value) const
	{
		return Array->Get(Index).Find(Value);
	}


	/**
	 * Finds an element which matches a predicate functor.
	 *
	 * @param Pred The functor to apply to each element.
	 *
	 * @return Pointer to the first element for which the predicate returns
	 *         true, or nullptr if none is found.
	 */
	template <typename Predicate>
	AttributeType* FindByPredicate(Predicate Pred) const
	{
		return Array->Get(Index).FindByPredicate(MoveTemp(Pred));
	}


	/**
	 * Finds an item by predicate.
	 *
	 * @param Pred The predicate to match.
	 *
	 * @returns Index to the first matching element, or INDEX_NONE if none is
	 *          found.
	 */
	template <typename Predicate>
	int32 IndexOfByPredicate(Predicate Pred) const
	{
		return Array->Get(Index).IndexOfByPredicate(MoveTemp(Pred));
	}


	/**
	 * Determines whether the array attribute contains a particular value
	 */
	bool Contains(const AttributeType& Value) const
	{
		return Array->Get(Index).Contains(Value);
	}


	/**
	 * Checks if this array contains element for which the predicate is true.
	 *
	 * @param Predicate to use
	 *
	 * @returns	True if found. False otherwise.
	 */
	template <typename Predicate>
	bool ContainsByPredicate(Predicate Pred) const
	{
		return Array->Get(Index).ContainsByPredicate(MoveTemp(Pred));
	}


	/**
	 * Sorts the array assuming < operator is defined for the item type.
	 */
	void Sort()
	{
		Array->Get(Index).Sort();
	}

	/**
	 * Sorts the array using the given predicate.
	 */
	template<typename Predicate>
	void SortByPredicate(Predicate Pred)
	{
		Array->Get(Index).template Sort<Predicate>(Pred);
	}


	/**
	 * Stable sorts the array assuming < operator is defined for the item type.
	 */
	void StableSort()
	{
		Array->Get(Index).StableSort();
	}


	/**
	 * Sorts the array using the given predicate.
	 */
	template<typename Predicate>
	void StableSortByPredicate(Predicate Pred)
	{
		Array->Get(Index).template StableSort<Predicate>(Pred);
	}

	/**
	 * Return a TArrayView representing this array attribute.
	 */
	TArrayView<AttributeType> ToArrayView() { return Array->Get(Index); }

	/**
	 * Implicitly coerce an array attribute to a TArrayView.
	 */
	operator TArrayView<AttributeType>() { return Array->Get(Index); }

public:
	/**
	* DO NOT USE DIRECTLY
	* STL-like iterators to enable range-based for loop support.
	*/
	inline AttributeType* begin() const { return GetData(); }
	inline AttributeType* end  () const { return GetData() + Num(); }

private:
	ArrayType* Array;
	int32 Index;
};


