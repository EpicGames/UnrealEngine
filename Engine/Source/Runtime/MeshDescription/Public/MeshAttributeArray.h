// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AttributeArrayContainer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "Delegates/IntegerSequence.h"
#include "HAL/PlatformCrt.h"
#include "Logging/StructuredLog.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "MeshElementRemappings.h"
#include "MeshTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Crc.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/TVariant.h"
#include "Serialization/Archive.h"
#include "Templates/CopyQualifiersFromTo.h"
#include "Templates/EnableIf.h"
#include "Templates/IsArray.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/NameTypes.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"

struct FElementID;


/**
 * List of attribute types which are supported.
 *
 * IMPORTANT NOTE: Do not reorder or remove any type from this tuple, or serialization will fail.
 * Types may be added at the end of this list as required.
 */
using AttributeTypes = TTuple
<
	FVector4f,
	FVector3f,
	FVector2f,
	float,
	int32,
	bool,
	FName,
	FTransform
>;


/**
 * Helper template which generates a TVariant of all supported attribute types.
 */
template <typename Tuple> struct TVariantFromTuple;

template <typename... Ts> struct TVariantFromTuple<TTuple<Ts...>> { using Type = TVariant<FEmptyVariantState, Ts...>; };


/**
 * Traits class to specify which attribute types can be bulk serialized.
 */
template <typename T> struct TIsBulkSerializable { static const bool Value = true; };
template <> struct TIsBulkSerializable<FName> { static const bool Value = false; };
template <> struct TIsBulkSerializable<FTransform> { static const bool Value = false; };


/**
 * This defines the container used to hold mesh element attributes of a particular name and index.
 * It is a simple TArray, so that all attributes are packed contiguously for each element ID.
 *
 * Note that the container may grow arbitrarily as new elements are inserted, but it will never be
 * shrunk as elements are removed. The only operations that will shrink the container are Initialize() and Remap().
 */
template <typename AttributeType>
class TMeshAttributeArrayBase
{
public:
	explicit TMeshAttributeArrayBase(uint32 InExtent = 1)
		: Extent(InExtent)
	{}

	/** Custom serialization for TMeshAttributeArrayBase. */
	template <typename T> friend typename TEnableIf<!TIsBulkSerializable<T>::Value, FArchive>::Type& operator<<( FArchive& Ar, TMeshAttributeArrayBase<T>& Array );
	template <typename T> friend typename TEnableIf<TIsBulkSerializable<T>::Value, FArchive>::Type& operator<<( FArchive& Ar, TMeshAttributeArrayBase<T>& Array );

	/** Return size of container */
	inline int32 Num() const { return Container.Num() / Extent; }

	/** Return base of data */
	UE_DEPRECATED(5.0, "This method will be removed.")
	inline const AttributeType* GetData() const { return Container.GetData(); }

	/** Initializes the array to the given size with the default value */
	inline void Initialize(const int32 ElementCount, const AttributeType& Default)
	{
		Container.Reset(ElementCount * Extent);
		if (ElementCount > 0)
		{
			Insert(ElementCount - 1, Default);
		}
	}

	void SetNum(const int32 ElementCount, const AttributeType& Default)
	{
		if (int32(ElementCount * Extent) < Container.Num())
		{
			// Setting to a lower number; just truncate the container
			Container.SetNum(ElementCount * Extent);
		}
		else
		{
			// Setting to a higher number; grow the container, inserting default value to end.
			Insert(ElementCount - 1, Default);
		}
	}

	uint32 GetHash(uint32 Crc = 0) const
	{
		return FCrc::MemCrc32(Container.GetData(), Container.Num() * sizeof(AttributeType), Crc);
	}

	/** Expands the array if necessary so that the passed element index is valid. Newly created elements will be assigned the default value. */
	void Insert(const int32 Index, const AttributeType& Default);

	/** Fills the index with the default value */
	void SetToDefault(const int32 Index, const AttributeType& Default)
	{
		for (uint32 I = 0; I < Extent; ++I)
		{
			Container[Index * Extent + I] = Default;
		}
	}

	/** Remaps elements according to the passed remapping table */
	void Remap(const TSparseArray<int32>& IndexRemap, const AttributeType& Default);

	/** Element accessors */
	UE_DEPRECATED(5.0, "Please use GetElementBase() instead.")
	inline const AttributeType& operator[](const int32 Index) const { return Container[Index]; }

	UE_DEPRECATED(5.0, "Please use GetElementBase() instead.")
	inline AttributeType& operator[](const int32 Index) { return Container[Index]; }

	inline const AttributeType* GetElementBase(const int32 Index) const { return &Container[Index * Extent]; }
	inline AttributeType* GetElementBase(const int32 Index) { return &Container[Index * Extent]; }

	inline uint32 GetExtent() const { return Extent; }

protected:
	/** The actual container, represented by a regular array */
	TArray<AttributeType> Container;

	/** Number of array elements in this attribute type */
	uint32 Extent;
};

// We have to manually implement this or else the default behavior will just hash FName ComparisonIndex and Number,
// which aren't deterministic
template<>
inline uint32 TMeshAttributeArrayBase<FName>::GetHash(uint32 Crc) const
{
	for (const FName& Item : Container)
	{
		const FString ItemStr = Item.ToString();
		Crc = FCrc::MemCrc32(*ItemStr, ItemStr.Len() * sizeof(TCHAR), Crc);
	}
	return Crc;
}

template <typename AttributeType>
void TMeshAttributeArrayBase<AttributeType>::Insert(const int32 Index, const AttributeType& Default)
{
	int32 EndIndex = (Index + 1) * Extent;
	if (EndIndex > Container.Num())
	{
		// If the index is off the end of the container, add as many elements as required to make it the last valid index.
		int32 StartIndex = Container.AddUninitialized(EndIndex - Container.Num());
		AttributeType* Data = Container.GetData() + StartIndex;

		// Construct added elements with the default value passed in

		while (StartIndex < EndIndex)
		{
			new(Data) AttributeType(Default);
			StartIndex++;
			Data++;
		}
	}
}

template <typename AttributeType>
void TMeshAttributeArrayBase<AttributeType>::Remap(const TSparseArray<int32>& IndexRemap, const AttributeType& Default)
{
	TMeshAttributeArrayBase NewAttributeArray(Extent);

	for (typename TSparseArray<int32>::TConstIterator It(IndexRemap); It; ++It)
	{
		const int32 OldElementIndex = It.GetIndex();
		const int32 NewElementIndex = IndexRemap[OldElementIndex];

		NewAttributeArray.Insert(NewElementIndex, Default);
		AttributeType* DestElementBase = NewAttributeArray.GetElementBase(NewElementIndex);
		AttributeType* SrcElementBase = GetElementBase(OldElementIndex);
		for (uint32 Index = 0; Index < Extent; ++Index)
		{
			DestElementBase[Index] = MoveTemp(SrcElementBase[Index]);
		}
	}

	Container = MoveTemp(NewAttributeArray.Container);
}

template <typename T>
inline typename TEnableIf<!TIsBulkSerializable<T>::Value, FArchive>::Type& operator<<( FArchive& Ar, TMeshAttributeArrayBase<T>& Array )
{
	if (Ar.IsLoading() &&
		Ar.CustomVer(FReleaseObjectVersion::GUID) != FReleaseObjectVersion::MeshDescriptionNewFormat &&
		Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::MeshDescriptionNewFormat)
	{
		Array.Extent = 1;
	}
	else
	{
		Ar << Array.Extent;
	}
	
	// A little bit prior to skeletal meshes storing their model data as mesh description, FTransform attributes were added but marked,
	// by default, as bulk-serializable, even though FTransform doesn't support it. Therefore, this serialization path only worked on empty 
	// FTransform attributes. However, there are still static mesh assets in the wild that contain empty FTransform attributes, and we need 
	// to be able to successfully load them -- hence this check.
	if constexpr (std::is_same_v<T, FTransform>)
	{
		if (Ar.IsLoading())
		{
			// This version check works because saved UStaticMesh assets set this on their archive, which is then inherited by the
			// mesh description bulk storage.
			const FCustomVersion* PossiblySavedVersion = Ar.GetCustomVersions().GetVersion(FFortniteMainBranchObjectVersion::GUID);
			if (PossiblySavedVersion && PossiblySavedVersion->Version < FFortniteMainBranchObjectVersion::MeshDescriptionForSkeletalMesh)
			{
				Array.Container.BulkSerialize( Ar );
				return Ar;
			}
		}
	}

	// Serialize types which aren't bulk serializable, which need to be serialized element-by-element
	Ar << Array.Container;
	return Ar;
}

template <>
inline TEnableIf<!TIsBulkSerializable<FName>::Value, FArchive>::Type& operator<<( FArchive& Ar, TMeshAttributeArrayBase<FName>& Array )
{
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::DynamicMeshAttributesTriangleLabels)
	{		
		// see non-specialized implementation
		if (Ar.IsLoading() &&
		   Ar.CustomVer(FReleaseObjectVersion::GUID) != FReleaseObjectVersion::MeshDescriptionNewFormat &&
		   Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::MeshDescriptionNewFormat)
		{
			Array.Extent = 1;
		}
		else
		{
			Ar << Array.Extent;
		}
		
		Ar << Array.Container;
		
		return Ar;
	}
	
	Ar << Array.Extent;
	
	enum class EIndexSize : uint8 { U8 = 0, U16 = 1, U32 = 2 };
	
	TArray<FName>& Names = Array.Container;
	
	if (Ar.IsLoading())
	{
		// read names count
		uint32 NumNames = 0;
		Ar << NumNames;
		
		constexpr uint32 ArrayMax = static_cast<uint32>(std::numeric_limits<TArray<FName>::SizeType>::max());
		if (!ensure(NumNames <= ArrayMax))
		{
			Ar.SetError();
			return Ar;
		}
		
		if (NumNames == 0)
		{
			Names.Reset();
			return Ar;
		}
		
		// ret unique names
		uint32 NumUniqueNames = 0;
		Ar << NumUniqueNames;
		
		if (!ensure(NumUniqueNames <= ArrayMax))
		{
			Ar.SetError();
			return Ar;
		}
		
		TArray<FName> UniqueNames;
		UniqueNames.Reserve(NumUniqueNames);
		
		for (uint32 Index = 0; Index < NumUniqueNames; ++Index)
		{
			FString NameStr;
			Ar << NameStr;
			UniqueNames.Add(FName(NameStr));
		}
	
		// single FName so assign the same for all triangles
		if (NumUniqueNames == 1)
		{
			Names.Init(UniqueNames[0], NumNames);
			return Ar;
		}
		
		// all names are unique 
		if (NumUniqueNames == NumNames)
		{
			Names = MoveTemp(UniqueNames);
			return Ar;
		}
		
		Names.Init(NAME_None, NumNames);
	
		// read index size
		EIndexSize IndexSize = EIndexSize::U8;
		Ar << IndexSize;
	
		// read indices relative to UniqueNames 
		for (uint32 Index = 0; Index < NumNames; ++Index)
		{
			uint32 NameIndex = 0;
			if (IndexSize == EIndexSize::U8)
			{
				uint8 v = 0;
				Ar << v;
				NameIndex = v;
			}
			else if (IndexSize == EIndexSize::U16)
			{
				uint16 v = 0;
				Ar << v;
				NameIndex = v;
			}
			else
			{
				Ar << NameIndex;
			}
	
			if (ensure(UniqueNames.IsValidIndex(NameIndex)))
			{
				Names[Index] = UniqueNames[NameIndex];
			}
		}
	}
	else
	{
		// write names count
		uint32 NumNames = Names.Num();
		Ar << NumNames;
		
		if (NumNames == 0)
		{
			return Ar;
		}
		
		// get unique names
		TMap<FName, uint32> NameToIndex;
		TArray<FString> UniqueNames;
		TArray<uint32> NameIndices;
		NameIndices.Init(0, NumNames);
	
		for (uint32 Index = 0; Index < NumNames; ++Index)
		{
			const FName Name = Names[Index];
			if (const uint32* Found = NameToIndex.Find(Name))
			{
				NameIndices[Index] = *Found;
			}
			else
			{
				NameIndices[Index] = UniqueNames.Num();
				NameToIndex.Add(Name, UniqueNames.Num());
				UniqueNames.Emplace(Name.ToString());
			}
		}
		
		// write unique count and names
		uint32 NumUniqueNames = UniqueNames.Num();
		Ar << NumUniqueNames;
		
		for (uint32 Index = 0; Index < NumUniqueNames; ++Index)
		{
			Ar << UniqueNames[Index];
		}
		
		// single FName or all names are unique so nothing else to do
		if (NumUniqueNames == 1 || NumUniqueNames == NumNames)
		{
			return Ar;
		}
	
		// choose a convenient index size and write it
		EIndexSize IndexSize = (NumUniqueNames <= MAX_uint8) ? EIndexSize::U8 : (NumUniqueNames <= MAX_uint16) ? EIndexSize::U16 : EIndexSize::U32;
		Ar << IndexSize;
	
		// write indices relative to UniqueNames 
		for (uint32 Index = 0; Index < NumNames; ++Index)
		{
			if (IndexSize == EIndexSize::U8)
			{
				uint8 v = static_cast<uint8>(NameIndices[Index]);
				Ar << v;
			}
			else if (IndexSize == EIndexSize::U16)
			{
				uint16 v = static_cast<uint16>(NameIndices[Index]);
				Ar << v;
			}
			else
			{
				Ar << NameIndices[Index];
			}
		}
	}
	
	return Ar;
}


template <typename T>
inline typename TEnableIf<TIsBulkSerializable<T>::Value, FArchive>::Type& operator<<( FArchive& Ar, TMeshAttributeArrayBase<T>& Array )
{
	if (Ar.IsLoading() &&
		Ar.CustomVer(FReleaseObjectVersion::GUID) != FReleaseObjectVersion::MeshDescriptionNewFormat &&
		Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::MeshDescriptionNewFormat)
	{
		Array.Extent = 1;
	}
	else
	{
		Ar << Array.Extent;
	}

	if( Ar.IsLoading() && Ar.CustomVer( FReleaseObjectVersion::GUID ) < FReleaseObjectVersion::MeshDescriptionNewSerialization )
	{
		// Legacy path for old format attribute arrays. BulkSerialize has a different format from regular serialization.
		Ar << Array.Container;
	}
	else
	{
		// Serialize types which are bulk serializable, i.e. which can be memcpy'd in bulk
		Array.Container.BulkSerialize( Ar );
	}

	return Ar;
}


/**
 * Flags specifying properties of an attribute
 */
enum class EMeshAttributeFlags : uint32
{
	None				= 0,
	Lerpable			= (1 << 0),		/** Attribute can be automatically lerped according to the value of 2 or 3 other attributes */
	AutoGenerated		= (1 << 1),		/** Attribute is auto-generated by importer or editable mesh, rather than representing an imported property */	
	Mergeable			= (1 << 2),		/** If all vertices' attributes are mergeable, and of near-equal value, they can be welded */
	Transient			= (1 << 3),		/** Attribute is not serialized */
	IndexReference		= (1 << 4),		/** Attribute is a reference to another element index */
	Mandatory			= (1 << 5),		/** Attribute is required in the mesh description */
};

ENUM_CLASS_FLAGS(EMeshAttributeFlags);


/**
 * This is the base class for an attribute array set.
 * An attribute array set is a container which holds attribute arrays, one per attribute index.
 * Many attributes have only one index, while others (such as texture coordinates) may want to define many.
 *
 * All attribute array set instances will be of derived types; this type exists for polymorphism purposes,
 * so that they can be managed by a generic TUniquePtr<FMeshAttributeArraySetBase>.
 *
 * In general, we avoid accessing them via virtual dispatch by insisting that their type be passed as
 * a template parameter in the accessor. This can be checked against the Type field to ensure that we are
 * accessing an instance by its correct type.
 */
class FMeshAttributeArraySetBase
{
public:
	/** Constructor */
	inline FMeshAttributeArraySetBase(const uint32 InType, const EMeshAttributeFlags InFlags, const int32 InNumberOfElements, const uint32 InExtent)
		: Type(InType),
		  Extent(InExtent),
		  NumElements(InNumberOfElements),
		  Flags(InFlags)
	{}

	/** Virtual interface */
	virtual ~FMeshAttributeArraySetBase() = default;
	virtual TUniquePtr<FMeshAttributeArraySetBase> Clone() const = 0;
	virtual void Insert(const int32 Index) = 0;
	virtual void Remove(const int32 Index) = 0;
	virtual void Initialize(const int32 Count) = 0;
	virtual void SetNumElements(const int32 Count) = 0;
	virtual uint32 GetHash() const = 0;
	virtual void Serialize(FArchive& Ar) = 0;
	virtual void Remap(const TSparseArray<int32>& IndexRemap) = 0;

//	UE_DEPRECATED(5.0, "Please use GetNumChannels().")
	virtual int32 GetNumIndices() const = 0;
//	UE_DEPRECATED(5.0, "Please use SetNumChannels().")
	virtual void SetNumIndices(const int32 NumIndices) = 0;
//	UE_DEPRECATED(5.0, "Please use InsertChannel().")
	virtual void InsertIndex(const int32 Index) = 0;
//	UE_DEPRECATED(5.0, "Please use RemoveChannel().")
	virtual void RemoveIndex(const int32 Index) = 0;

	virtual int32 GetNumChannels() const = 0;
	virtual void SetNumChannels(const int32 NumChannels) = 0;
	virtual void InsertChannel(const int32 Index) = 0;
	virtual void RemoveChannel(const int32 Index) = 0;

	/** Determine whether this attribute array set is of the given type */
	template <typename T>
	inline bool HasType() const
	{
		return TTupleIndex<T, AttributeTypes>::Value == Type;
	}

	/** Get the type index of this attribute array set */
	inline uint32 GetType() const { return Type; }

	/** Get the type extent of this attribute array set */
	inline uint32 GetExtent() const { return Extent; }

	/** Get the flags for this attribute array set */
	inline EMeshAttributeFlags GetFlags() const { return Flags; }

	/** Set the flags for this attribute array set */
	inline void SetFlags(const EMeshAttributeFlags InFlags) { Flags = InFlags; }

	/** Return number of elements each attribute index has */
	inline int32 GetNumElements() const { return NumElements; }

protected:
	/** Type of the attribute array (based on the tuple element index from AttributeTypes) */
	uint32 Type;

	/** Extent of the type, i.e. the number of array elements it consists of */
	uint32 Extent;

	/** Number of elements in each index */
	int32 NumElements;

	/** Implementation-defined attribute name flags */
	EMeshAttributeFlags Flags;
};


/**
 * This is a type-specific attribute array, which is actually instanced in the attribute set.
 */
template <typename AttributeType>
class TMeshAttributeArraySet final : public FMeshAttributeArraySetBase
{
	static_assert(!TIsArray<AttributeType>::Value, "TMeshAttributeArraySet must take a simple type.");

	using Super = FMeshAttributeArraySetBase;

public:
	/** Constructors */
	inline explicit TMeshAttributeArraySet(const int32 Extent = 1)
		: Super(TTupleIndex<AttributeType, AttributeTypes>::Value, EMeshAttributeFlags::None, 0, Extent)
	{}

	inline explicit TMeshAttributeArraySet(const int32 NumberOfChannels, const AttributeType& InDefaultValue, const EMeshAttributeFlags InFlags, const int32 InNumberOfElements, const uint32 Extent)
		: Super(TTupleIndex<AttributeType, AttributeTypes>::Value, InFlags, InNumberOfElements, Extent),
		  DefaultValue(InDefaultValue)
	{
		SetNumChannels(NumberOfChannels);
	}

	/** Creates a copy of itself and returns a TUniquePtr to it */
	virtual TUniquePtr<FMeshAttributeArraySetBase> Clone() const override
	{
		return MakeUnique<TMeshAttributeArraySet>(*this);
	}

	/** Insert the element at the given index */
	virtual void Insert(const int32 Index) override
	{
		for (TMeshAttributeArrayBase<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			ArrayForChannel.Insert(Index, DefaultValue);
		}

		NumElements = FMath::Max(NumElements, Index + 1);
	}

	/** Remove the element at the given index, replacing it with a default value */
	virtual void Remove(const int32 Index) override
	{
		for (TMeshAttributeArrayBase<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			ArrayForChannel.SetToDefault(Index, DefaultValue);
		}
	}

	/** Sets the number of elements to the exact number provided, and initializes them to the default value */
	virtual void Initialize(const int32 Count) override
	{
		NumElements = Count;
		for (TMeshAttributeArrayBase<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			ArrayForChannel.Initialize(Count, DefaultValue);
		}
	}

	/** Sets the number of elements to the exact number provided, preserving existing elements if the number is bigger */
	virtual void SetNumElements(const int32 Count) override
	{
		NumElements = Count;
		for (TMeshAttributeArrayBase<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			ArrayForChannel.SetNum(Count, DefaultValue);
		}
	}

	virtual uint32 GetHash() const override
	{
		uint32 CrcResult = 0;
		for (const TMeshAttributeArrayBase<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			CrcResult = ArrayForChannel.GetHash(CrcResult);
		}
		return CrcResult;
	}

	/** Polymorphic serialization */
	virtual void Serialize(FArchive& Ar) override
	{
		Ar << (*this);
	}

	/** Performs an element index remap according to the passed array */
	virtual void Remap(const TSparseArray<int32>& IndexRemap) override
	{
		for (TMeshAttributeArrayBase<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			ArrayForChannel.Remap(IndexRemap, DefaultValue);
			NumElements = ArrayForChannel.Num();
		}
	}

	UE_DEPRECATED(5.0, "Please use GetNumChannels().")
	virtual inline int32 GetNumIndices() const override { return GetNumChannels(); }

	/** Return number of channels this attribute has */
	virtual inline int32 GetNumChannels() const override { return ArrayForChannels.Num(); }

	UE_DEPRECATED(5.0, "Please use SetNumChannels().")
	virtual void SetNumIndices(const int32 NumIndices) override { SetNumChannels(NumIndices); }

	/** Sets number of channels this attribute has */
	virtual void SetNumChannels(const int32 NumChannels) override
	{
		if (NumChannels < ArrayForChannels.Num())
		{
			ArrayForChannels.SetNum(NumChannels);
			return;
		}

		while (ArrayForChannels.Num() < NumChannels)
		{
			TMeshAttributeArrayBase<AttributeType>& Array = ArrayForChannels.Emplace_GetRef(Extent);
			Array.Initialize(NumElements, DefaultValue);
		}
	}

	UE_DEPRECATED(5.0, "Please use InsertChannel().")
	virtual void InsertIndex(const int32 Index) override
	{
		InsertChannel(Index);
	}

	/** Insert a new attribute channel */
	virtual void InsertChannel(const int32 Index) override
	{
		TMeshAttributeArrayBase<AttributeType>& Array = ArrayForChannels.EmplaceAt_GetRef(Index, Extent);
		Array.Initialize(NumElements, DefaultValue);
	}

	UE_DEPRECATED(5.0, "Please use RemoveChannel().")
	virtual void RemoveIndex(const int32 Index) override
	{
		RemoveChannel(Index);
	}

	/** Remove the channel at the given index */
	virtual void RemoveChannel(const int32 Index) override
	{
		ArrayForChannels.RemoveAt(Index);
	}


	UE_DEPRECATED(5.0, "Please use GetArrayForChannel().")
	inline const TMeshAttributeArrayBase<AttributeType>& GetArrayForIndex( const int32 Index ) const { return ArrayForChannels[ Index ]; }
	UE_DEPRECATED(5.0, "Please use GetArrayForChannel().")
	inline TMeshAttributeArrayBase<AttributeType>& GetArrayForIndex( const int32 Index ) { return ArrayForChannels[ Index ]; }

	/** Return the TMeshAttributeArrayBase corresponding to the given attribute channel */
	inline const TMeshAttributeArrayBase<AttributeType>& GetArrayForChannel( const int32 Index ) const { return ArrayForChannels[ Index ]; }
	inline TMeshAttributeArrayBase<AttributeType>& GetArrayForChannel( const int32 Index ) { return ArrayForChannels[ Index ]; }

	/** Return default value for this attribute type */
	inline AttributeType GetDefaultValue() const { return DefaultValue; }

	/** Serializer */
	friend FArchive& operator<<(FArchive& Ar, TMeshAttributeArraySet& AttributeArraySet)
	{
		Ar << AttributeArraySet.NumElements;
		Ar << AttributeArraySet.ArrayForChannels;
		Ar << AttributeArraySet.DefaultValue;
		Ar << AttributeArraySet.Flags;

		return Ar;
	}

protected:
	/** An array of MeshAttributeArrays, one per channel */
	TArray<TMeshAttributeArrayBase<AttributeType>, TInlineAllocator<1>> ArrayForChannels;

	/** The default value for an attribute of this name */
	AttributeType DefaultValue;
};


/**
* This is a type-specific attribute array, which is actually instanced in the attribute set.
*/
template <typename AttributeType>
class TMeshUnboundedAttributeArraySet final : public FMeshAttributeArraySetBase
{
	using Super = FMeshAttributeArraySetBase;

public:
	/** Constructors */
	inline TMeshUnboundedAttributeArraySet()
		: Super(TTupleIndex<AttributeType, AttributeTypes>::Value, EMeshAttributeFlags::None, 0, 0)
	{}

	inline explicit TMeshUnboundedAttributeArraySet(const int32 NumberOfChannels, const AttributeType& InDefaultValue, const EMeshAttributeFlags InFlags, const int32 InNumberOfElements)
		: Super(TTupleIndex<AttributeType, AttributeTypes>::Value, InFlags, InNumberOfElements, 0),
		DefaultValue(InDefaultValue)
	{
		SetNumChannels(NumberOfChannels);
	}

	/** Creates a copy of itself and returns a TUniquePtr to it */
	virtual TUniquePtr<FMeshAttributeArraySetBase> Clone() const override
	{
		return MakeUnique<TMeshUnboundedAttributeArraySet>(*this);
	}

	/** Insert the element at the given index */
	virtual void Insert(const int32 Index) override
	{
		for (TAttributeArrayContainer<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			ArrayForChannel.Insert(Index, DefaultValue);
		}

		NumElements = FMath::Max(NumElements, Index + 1);
	}

	/** Remove the element at the given index, replacing it with a default value */
	virtual void Remove(const int32 Index) override
	{
		for (TAttributeArrayContainer<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			ArrayForChannel.SetToDefault(Index, DefaultValue);
		}
	}

	/** Sets the number of elements to the exact number provided, and initializes them to the default value */
	virtual void Initialize(const int32 Count) override
	{
		NumElements = Count;
		for (TAttributeArrayContainer<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			ArrayForChannel.Initialize(Count, DefaultValue);
		}
	}

	/** Sets the number of elements to the exact number provided, preserving existing elements if the number is bigger */
	virtual void SetNumElements(const int32 Count) override
	{
		NumElements = Count;
		for (TAttributeArrayContainer<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			ArrayForChannel.SetNum(Count, DefaultValue);
		}
	}

	virtual uint32 GetHash() const override
	{
		uint32 CrcResult = 0;
		for (const TAttributeArrayContainer<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			CrcResult = ArrayForChannel.GetHash(CrcResult);
		}
		return CrcResult;
	}

	/** Polymorphic serialization */
	virtual void Serialize(FArchive& Ar) override
	{
		Ar << (*this);
	}

	/** Performs an element index remap according to the passed array */
	virtual void Remap(const TSparseArray<int32>& IndexRemap) override
	{
		for (TAttributeArrayContainer<AttributeType>& ArrayForChannel : ArrayForChannels)
		{
			ArrayForChannel.Remap(IndexRemap, DefaultValue);
			NumElements = ArrayForChannel.Num();
		}
	}

	UE_DEPRECATED(5.0, "Please use GetNumChannels().")
	virtual inline int32 GetNumIndices() const override { return GetNumChannels(); }

	/** Return number of channels this attribute has */
	virtual inline int32 GetNumChannels() const override { return ArrayForChannels.Num(); }

	UE_DEPRECATED(5.0, "Please use SetNumChannels().")
	virtual void SetNumIndices(const int32 NumIndices) override { SetNumChannels(NumIndices); }

	/** Sets number of channels this attribute has */
	virtual void SetNumChannels(const int32 NumChannels) override
	{
		if (NumChannels < ArrayForChannels.Num())
		{
			ArrayForChannels.SetNum(NumChannels);
			return;
		}

		while (ArrayForChannels.Num() < NumChannels)
		{
			TAttributeArrayContainer<AttributeType>& Array = ArrayForChannels.Emplace_GetRef(DefaultValue);
			Array.Initialize(NumElements, DefaultValue);
		}
	}

	UE_DEPRECATED(5.0, "Please use InsertChannel().")
	virtual void InsertIndex(const int32 Index) override
	{
		InsertChannel(Index);
	}

	/** Insert a new attribute channel */
	virtual void InsertChannel(const int32 Index) override
	{
		TAttributeArrayContainer<AttributeType>& Array = ArrayForChannels.EmplaceAt_GetRef(Index, DefaultValue);
		Array.Initialize(NumElements, DefaultValue);
	}

	UE_DEPRECATED(5.0, "Please use RemoveChannel().")
	virtual void RemoveIndex(const int32 Index) override
	{
		RemoveChannel(Index);
	}

	/** Remove the channel at the given index */
	virtual void RemoveChannel(const int32 Index) override
	{
		ArrayForChannels.RemoveAt(Index);
	}

	/** Return the TMeshAttributeArrayBase corresponding to the given attribute channel */
	inline const TAttributeArrayContainer<AttributeType>& GetArrayForChannel( const int32 Index ) const { return ArrayForChannels[ Index ]; }
	inline TAttributeArrayContainer<AttributeType>& GetArrayForChannel( const int32 Index ) { return ArrayForChannels[ Index ]; }

	/** Return default value for this attribute type */
	inline AttributeType GetDefaultValue() const { return DefaultValue; }

	/** Serializer */
	friend FArchive& operator<<(FArchive& Ar, TMeshUnboundedAttributeArraySet& AttributeArraySet)
	{
		Ar << AttributeArraySet.NumElements;
		Ar << AttributeArraySet.ArrayForChannels;
		Ar << AttributeArraySet.DefaultValue;
		Ar << AttributeArraySet.Flags;

		return Ar;
	}

protected:
	/** An array of UnboundedArrays, one per channel */
	TArray<TAttributeArrayContainer<AttributeType>, TInlineAllocator<1>> ArrayForChannels;

	/** The default value for an attribute of this name */
	AttributeType DefaultValue;
};


/**
 * Define type traits for different kinds of mesh attributes.
 * 
 * There are three type of attributes:
 * - simple values (T)
 * - fixed size arrays of values (TArrayView<T>)
 * - variable size arrays of values (TArrayAttribute<T>)
 *
 * Each of these corresponds to a different type of TMeshAttributesRef and TMeshAttributeArraySet.
 */
template <typename T>
struct TMeshAttributesRefTypeBase
{
	using AttributeType = T;
	using RealAttributeType = std::conditional_t<TIsDerivedFrom<AttributeType, FElementID>::Value, int32, AttributeType>;
};

template <typename T>
struct TMeshAttributesRefType : TMeshAttributesRefTypeBase<T>
{
	static const uint32 MinExpectedExtent = 1;
	static const uint32 MaxExpectedExtent = 1;
	using RefType = T;
	using ConstRefType = const T;
	using NonConstRefType = std::remove_cv_t<T>;
};

template <typename T>
struct TMeshAttributesRefType<TArrayView<T>> : TMeshAttributesRefTypeBase<T>
{
	static const uint32 MinExpectedExtent = 0;
	static const uint32 MaxExpectedExtent = 0xFFFFFFFF;
	using RefType = TArrayView<T>;
	using ConstRefType = TArrayView<const T>;
	using NonConstRefType = TArrayView<std::remove_cv_t<T>>;
};

template <typename T>
struct TMeshAttributesRefType<TArrayAttribute<T>> : TMeshAttributesRefTypeBase<T>
{
	static const uint32 MinExpectedExtent = 0;
	static const uint32 MaxExpectedExtent = 0;
	using RefType = TArrayAttribute<T>;
	using ConstRefType = TArrayAttribute<const T>;
	using NonConstRefType = TArrayAttribute<std::remove_cv_t<T>>;
};


/**
 * Additional type traits for registering different attributes.
 * When registering, we need to specify the attribute type with a concrete element count if necessary, i.e.
 * 
 * - simple values (T)
 * - fixed size arrays of values (T[N])
 * - variable size arrays of values (T[])
 */
template <typename T>
struct TMeshAttributesRegisterType : TMeshAttributesRefType<T>
{
	static const uint32 Extent = 1;
};

template <typename T, SIZE_T N>
struct TMeshAttributesRegisterType<T[N]> : TMeshAttributesRefType<TArrayView<T>>
{
	static const uint32 Extent = N;
};

template <typename T>
struct TMeshAttributesRegisterType<T[]> : TMeshAttributesRefType<TArrayAttribute<T>>
{
	static const uint32 Extent = 0;
};



/**
 * This is the class used to access attribute values.
 * It is a proxy object to a TMeshAttributeArraySet<> and should be passed by value.
 * It is valid for as long as the owning FMeshDescription exists.
 */
template <typename ElementIDType, typename AttributeType>
class TMeshAttributesRef;

template <typename ElementIDType, typename AttributeType>
using TMeshAttributesConstRef = TMeshAttributesRef<ElementIDType, typename TMeshAttributesRefType<AttributeType>::ConstRefType>;

template <typename AttributeType>
using TMeshAttributesArray = TMeshAttributesRef<int32, AttributeType>;

template <typename AttributeType>
using TMeshAttributesConstArray = TMeshAttributesRef<int32, typename TMeshAttributesRefType<AttributeType>::ConstRefType>;


// This is the default implementation which handles simple attributes, i.e. those of a simple type T.
// There are partial specializations which handle compound attributes below, i.e. those accessed via TArrayView<T> or TAttributeArray<T>
template <typename ElementIDType, typename AttributeType>
class TMeshAttributesRef
{
	template <typename T, typename U> friend class TMeshAttributesRef;

public:
	using BaseArrayType = TCopyQualifiersFromTo_T<AttributeType, FMeshAttributeArraySetBase>;
	using ArrayType = TCopyQualifiersFromTo_T<AttributeType, TMeshAttributeArraySet<std::remove_cv_t<AttributeType>>>;

	/** Constructor taking a pointer to a FMeshAttributeArraySetBase */
	explicit TMeshAttributesRef(BaseArrayType* InArrayPtr = nullptr, uint32 InExtent = 1)
		: ArrayPtr(InArrayPtr)
	{}

	/** Implicitly construct a TMeshAttributesRef-to-const from a regular one */
	template <typename SrcAttributeType,
			  typename DestAttributeType = AttributeType,
			  typename TEnableIf<std::is_const_v<DestAttributeType>, int>::Type = 0,
			  typename TEnableIf<!std::is_const_v<SrcAttributeType>, int>::Type = 0>
	TMeshAttributesRef(const TMeshAttributesRef<ElementIDType, SrcAttributeType>& InRef)
		: ArrayPtr(InRef.ArrayPtr)
	{}

	/** Implicitly construct a TMeshAttributesRef from a TMeshAttributesArray **/
	template <typename IDType = ElementIDType,
			  typename TEnableIf<!std::is_same_v<IDType, int32>, int>::Type = 0>
	TMeshAttributesRef(const TMeshAttributesRef<int32, AttributeType>& InRef)
		: ArrayPtr(InRef.ArrayPtr)
	{}

	/** Implicitly construct a TMeshAttributesRef-to-const from a TMeshAttributesArray */
	template <typename SrcAttributeType,
			  typename DestAttributeType = AttributeType,
			  typename IDType = ElementIDType,
			  typename TEnableIf<!std::is_same_v<IDType, int32>, int>::Type = 0,
			  typename TEnableIf<std::is_const_v<DestAttributeType>, int>::Type = 0,
			  typename TEnableIf<!std::is_const_v<SrcAttributeType>, int>::Type = 0>
	TMeshAttributesRef(const TMeshAttributesRef<int32, SrcAttributeType>& InRef)
		: ArrayPtr(InRef.ArrayPtr)
	{}

	/** Access elements from attribute channel 0 */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	AttributeType& operator[](const ElementIDType ElementID) const
	{
		return static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(0).GetElementBase(ElementID.GetValue())[0];
	}

	/** Get the element with the given ID and channel */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	AttributeType Get(const ElementIDType ElementID, const int32 Channel = 0) const
	{
		return static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).GetElementBase(ElementID.GetValue())[0];
	}

	AttributeType& operator[](int32 ElementIndex) const
	{
		return static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(0).GetElementBase(ElementIndex)[0];
	}

	AttributeType Get(int32 ElementIndex, const int32 Channel = 0) const
	{
		return static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).GetElementBase(ElementIndex)[0];
	}

	TArrayView<AttributeType> GetArrayView(int32 ElementIndex, const int32 Channel = 0) const
	{
		return TArrayView<AttributeType>(static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).GetElementBase(ElementIndex), 1);
	}

	TArrayView<AttributeType> GetRawArray(const int32 AttributeChannel = 0) const
	{
		if (ArrayPtr == nullptr || GetNumElements() == 0)
		{
			return TArrayView<AttributeType>();
		}

		AttributeType* Element = static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(AttributeChannel).GetElementBase(0);
		return TArrayView<AttributeType>(Element, GetNumElements());
	}

	/** Return whether the reference is valid or not */
	bool IsValid() const { return (ArrayPtr != nullptr); }

	/** Return default value for this attribute type */
	AttributeType GetDefaultValue() const { return static_cast<ArrayType*>(ArrayPtr)->GetDefaultValue(); }

	UE_DEPRECATED(5.0, "Please use GetNumChannels().")
	int32 GetNumIndices() const
	{
		return static_cast<ArrayType*>(ArrayPtr)->ArrayType::GetNumChannels();	// note: override virtual dispatch
	}

	/** Return number of indices this attribute has */
	int32 GetNumChannels() const
	{
		return static_cast<ArrayType*>(ArrayPtr)->ArrayType::GetNumChannels();	// note: override virtual dispatch
	}

	/** Get the number of elements in this attribute array */
	int32 GetNumElements() const
	{
		return ArrayPtr->GetNumElements();
	}

	/** Get the flags for this attribute array set */
	EMeshAttributeFlags GetFlags() const { return ArrayPtr->GetFlags(); }

	/** Get the extent for this attribute type */
	uint32 GetExtent() const { return 1; }

	/** Set the element with the given ID and index 0 to the provided value */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	void Set(const ElementIDType ElementID, const AttributeType& Value) const
	{
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(0).GetElementBase(ElementID.GetValue())[0] = Value;
	}

	/** Set the element with the given ID and channel to the provided value */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	void Set(const ElementIDType ElementID, const int32 Channel, const AttributeType& Value) const
	{
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).GetElementBase(ElementID.GetValue())[0] = Value;
	}

	void Set(int32 ElementIndex, const AttributeType& Value) const
	{
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(0).GetElementBase(ElementIndex)[0] = Value;
	}

	void Set(int32 ElementIndex, const int32 Channel, const AttributeType& Value) const
	{
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).GetElementBase(ElementIndex)[0] = Value;
	}

	void SetArrayView(int32 ElementIndex, TArrayView<const AttributeType> Value) const
	{
		check(Value.Num() == 1);
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(0).GetElementBase(ElementIndex)[0] = Value[0];
	}

	void SetArrayView(int32 ElementIndex, const int32 Channel, TArrayView<const AttributeType> Value) const
	{
		check(Value.Num() == 1);
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).GetElementBase(ElementIndex)[0] = Value[0];
	}

	/** Copies the given attribute array and channel to this channel */
	void Copy(TMeshAttributesRef<ElementIDType, const AttributeType> Src, const int32 DestChannel = 0, const int32 SrcChannel = 0);

	UE_DEPRECATED(5.0, "Please use SetNumChannels().")
	void SetNumIndices(const int32 NumChannels) const
	{
		static_cast<ArrayType*>(ArrayPtr)->ArrayType::SetNumChannels(NumChannels);	// note: override virtual dispatch
	}

	/** Sets number of channels this attribute has */
	void SetNumChannels(const int32 NumChannels) const
	{
		static_cast<ArrayType*>(ArrayPtr)->ArrayType::SetNumChannels(NumChannels);	// note: override virtual dispatch
	}

	UE_DEPRECATED(5.0, "Please use InsertChannel().")
	void InsertIndex(const int32 Index) const
	{
		static_cast<ArrayType*>(ArrayPtr)->ArrayType::InsertChannel(Index);		// note: override virtual dispatch
	}

	/** Inserts an attribute channel */
	void InsertChannel(const int32 Channel) const
	{
		static_cast<ArrayType*>(ArrayPtr)->ArrayType::InsertChannel(Channel);		// note: override virtual dispatch
	}

	UE_DEPRECATED(5.0, "Please use RemoveChannel().")
	void RemoveIndex(const int32 Index) const
	{
		static_cast<ArrayType*>(ArrayPtr)->ArrayType::RemoveChannel(Index);		// note: override virtual dispatch
	}

	/** Removes an attribute channel */
	void RemoveChannel(const int32 Channel) const
	{
		static_cast<ArrayType*>(ArrayPtr)->ArrayType::RemoveChannel(Channel);		// note: override virtual dispatch
	}

private:
	BaseArrayType* ArrayPtr;
};

template <typename ElementIDType, typename AttributeType>
void TMeshAttributesRef<ElementIDType, AttributeType>::Copy(TMeshAttributesRef<ElementIDType, const AttributeType> Src, const int32 DestChannel, const int32 SrcChannel)
{
	check(Src.IsValid());
	const TMeshAttributeArrayBase<AttributeType>& SrcArray = static_cast<const ArrayType*>(Src.ArrayPtr)->GetArrayForChannel(SrcChannel);
	TMeshAttributeArrayBase<AttributeType>& DestArray = static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(DestChannel);
	const int32 Num = FMath::Min(SrcArray.Num(), DestArray.Num());
	for (int32 Index = 0; Index < Num; Index++)
	{
		DestArray.GetElementBase(Index)[0] = SrcArray.GetElementBase(Index)[0];
	}
}


template <typename ElementIDType, typename AttributeType>
class TMeshAttributesRef<ElementIDType, TArrayView<AttributeType>>
{
	template <typename T, typename U> friend class TMeshAttributesRef;

public:
	using BaseArrayType = TCopyQualifiersFromTo_T<AttributeType, FMeshAttributeArraySetBase>;
	using BoundedArrayType = TCopyQualifiersFromTo_T<AttributeType, TMeshAttributeArraySet<std::remove_cv_t<AttributeType>>>;
	using UnboundedArrayType = TCopyQualifiersFromTo_T<AttributeType, TMeshUnboundedAttributeArraySet<std::remove_cv_t<AttributeType>>>;

	/** Constructor taking a pointer to a TMeshAttributeArraySet */
	explicit TMeshAttributesRef(BaseArrayType* InArrayPtr = nullptr, uint32 InExtent = 1)
		: ArrayPtr(InArrayPtr),
		  Extent(InExtent)
	{}

	/** Implicitly construct a TMeshAttributesRef-to-const from a regular one */
	template <typename SrcAttributeType,
			  typename DestAttributeType = AttributeType,
			  typename TEnableIf<std::is_const_v<DestAttributeType>, int>::Type = 0,
			  typename TEnableIf<!std::is_const_v<SrcAttributeType>, int>::Type = 0>
	TMeshAttributesRef(const TMeshAttributesRef<ElementIDType, TArrayView<SrcAttributeType>>& InRef)
		: ArrayPtr(InRef.ArrayPtr),
		  Extent(InRef.Extent)
	{}

	/** Implicitly construct a TMeshAttributesRef from a TMeshAttributesArray **/
	template <typename IDType = ElementIDType,
			  typename TEnableIf<!std::is_same_v<IDType, int32>, int>::Type = 0>
	TMeshAttributesRef(const TMeshAttributesRef<int32, TArrayView<AttributeType>>& InRef)
		: ArrayPtr(InRef.ArrayPtr),
		  Extent(InRef.Extent)
	{}

	/** Implicitly construct a TMeshAttributesRef-to-const from a TMeshAttributesArray */
	template <typename SrcAttributeType,
			  typename DestAttributeType = AttributeType,
			  typename IDType = ElementIDType,
			  typename TEnableIf<!std::is_same_v<IDType, int32>, int>::Type = 0,
			  typename TEnableIf<std::is_const_v<DestAttributeType>, int>::Type = 0,
			  typename TEnableIf<!std::is_const_v<SrcAttributeType>, int>::Type = 0>
	TMeshAttributesRef(const TMeshAttributesRef<int32, TArrayView<SrcAttributeType>>& InRef)
		: ArrayPtr(InRef.ArrayPtr),
		  Extent(InRef.Extent)
	{}


	/** Access elements from attribute channel 0 */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	TArrayView<AttributeType> operator[](const ElementIDType ElementID) const
	{
		if (Extent > 0)
		{
			AttributeType* Element = static_cast<BoundedArrayType*>(ArrayPtr)->GetArrayForChannel(0).GetElementBase(ElementID.GetValue());
			return TArrayView<AttributeType>(Element, Extent);
		}
		else
		{
			return static_cast<UnboundedArrayType*>(ArrayPtr)->GetArrayForChannel(0).Get(ElementID.GetValue());
		}
	}

	/** Get the element with the given ID and channel */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	TArrayView<AttributeType> Get(const ElementIDType ElementID, const int32 Channel = 0) const
	{
		if (Extent > 0)
		{
			AttributeType* Element = static_cast<BoundedArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).GetElementBase(ElementID.GetValue());
			return TArrayView<AttributeType>(Element, Extent);
		}
		else
		{
			return static_cast<UnboundedArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).Get(ElementID.GetValue());
		}
	}

	TArrayView<AttributeType> operator[](int32 ElementIndex) const
	{
		if (Extent > 0)
		{
			AttributeType* Element = static_cast<BoundedArrayType*>(ArrayPtr)->GetArrayForChannel(0).GetElementBase(ElementIndex);
			return TArrayView<AttributeType>(Element, Extent);
		}
		else
		{
			return static_cast<UnboundedArrayType*>(ArrayPtr)->GetArrayForChannel(0).Get(ElementIndex);
		}
	}

	TArrayView<AttributeType> Get(int32 ElementIndex, const int32 Channel = 0) const
	{
		if (Extent > 0)
		{
			AttributeType* Element = static_cast<BoundedArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).GetElementBase(ElementIndex);
			return TArrayView<AttributeType>(Element, Extent);
		}
		else
		{
			return static_cast<UnboundedArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).Get(ElementIndex);
		}
	}

	TArrayView<AttributeType> GetArrayView(int32 ElementIndex, const int32 Channel = 0) const
	{
		if (Extent > 0)
		{
			return TArrayView<AttributeType>(static_cast<BoundedArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).GetElementBase(ElementIndex), Extent);
		}
		else
		{
			return static_cast<UnboundedArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).Get(ElementIndex);
		}
	}

	TArrayView<AttributeType> GetRawArray(const int32 ChannelIndex = 0) const
	{
		// Can't get the attribute set raw array for unbounded arrays because they are chunked
		check(Extent > 0);

		if (ArrayPtr == nullptr || GetNumElements() == 0)
		{
			return TArrayView<AttributeType>();
		}

		AttributeType* Element = static_cast<BoundedArrayType*>(ArrayPtr)->GetArrayForChannel(ChannelIndex).GetElementBase(0);
		return TArrayView<AttributeType>(Element, GetNumElements() * Extent);
	}

	/** Return whether the reference is valid or not */
	bool IsValid() const { return (ArrayPtr != nullptr); }

	/** Return default value for this attribute type */
	AttributeType GetDefaultValue() const { return static_cast<BoundedArrayType*>(ArrayPtr)->GetDefaultValue(); }

	UE_DEPRECATED(5.0, "Please use GetNumChannels().")
	int32 GetNumIndices() const
	{
		return ArrayPtr->GetNumChannels();
	}

	/** Return number of channels this attribute has */
	int32 GetNumChannels() const
	{
		return ArrayPtr->GetNumChannels();
	}

	/** Get the number of elements in this attribute array */
	int32 GetNumElements() const
	{
		return ArrayPtr->GetNumElements();
	}

	/** Get the flags for this attribute array set */
	EMeshAttributeFlags GetFlags() const { return ArrayPtr->GetFlags(); }

	/** Return the extent of this attribute, i.e. the number of array elements it comprises */
	uint32 GetExtent() const { return Extent; }

	/** Set the element with the given ID and index 0 to the provided value */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	void Set(const ElementIDType ElementID, TArrayView<const AttributeType> Value) const
	{
		TArrayView<AttributeType> Elements = Get(ElementID);
		check(Value.Num() == Elements.Num());
		for (uint32 Index = 0; Index < Extent; Index++)
		{
			Elements[Index] = Value[Index];
		}
	}

	/** Set the element with the given ID and channel to the provided value */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	void Set(const ElementIDType ElementID, const int32 Channel, TArrayView<const AttributeType> Value) const
	{
		TArrayView<AttributeType> Elements = Get(ElementID, Channel);
		check(Value.Num() == Elements.Num());
		for (uint32 Index = 0; Index < Extent; Index++)
		{
			Elements[Index] = Value[Index];
		}
	}

	void Set(int32 ElementIndex, TArrayView<const AttributeType> Value) const
	{
		TArrayView<AttributeType> Elements = Get(ElementIndex);
		check(Value.Num() == Elements.Num());
		for (uint32 Index = 0; Index < Extent; Index++)
		{
			Elements[Index] = Value[Index];
		}
	}

	void Set(int32 ElementIndex, const int32 Channel, TArrayView<const AttributeType> Value) const
	{
		TArrayView<AttributeType> Elements = Get(ElementIndex, Channel);
		check(Value.Num() == Elements.Num());
		for (uint32 Index = 0; Index < Extent; Index++)
		{
			Elements[Index] = Value[Index];
		}
	}

	void SetArrayView(int32 ElementIndex, TArrayView<const AttributeType> Value) const
	{
		Set(ElementIndex, Value);
	}

	void SetArrayView(int32 ElementIndex, const int32 Channel, TArrayView<const AttributeType> Value) const
	{
		Set(ElementIndex, Channel, Value);
	}

	/** Copies the given attribute array and index to this index */
	void Copy(TMeshAttributesConstRef<ElementIDType, TArrayView<AttributeType>> Src, const int32 DestChannel = 0, const int32 SrcChannel = 0);

	UE_DEPRECATED(5.0, "Please use SetNumChannels().")
	void SetNumIndices(const int32 NumChannels) const
	{
		ArrayPtr->SetNumChannels(NumChannels);
	}

	/** Sets number of channels this attribute has */
	void SetNumChannels(const int32 NumChannels) const
	{
		ArrayPtr->SetNumChannels(NumChannels);
	}

	UE_DEPRECATED(5.0, "Please use InsertChannel().")
	void InsertIndex(const int32 Index) const
	{
		ArrayPtr->InsertChannel(Index);
	}

	/** Inserts an attribute channel */
	void InsertChannel(const int32 Channel) const
	{
		ArrayPtr->InsertChannel(Channel);
	}

	UE_DEPRECATED(5.0, "Please use RemoveChannel().")
	void RemoveIndex(const int32 Index) const
	{
		ArrayPtr->RemoveChannel(Index);
	}

	/** Removes an attribute channel */
	void RemoveChannel(const int32 Channel) const
	{
		ArrayPtr->RemoveChannel(Channel);
	}

private:
	BaseArrayType* ArrayPtr;
	uint32 Extent;
};

template <typename ElementIDType, typename AttributeType>
void TMeshAttributesRef<ElementIDType, TArrayView<AttributeType>>::Copy(TMeshAttributesConstRef<ElementIDType, TArrayView<AttributeType>> Src, const int32 DestChannel, const int32 SrcChannel)
{
	check(Extent > 0);
	check(Src.IsValid());
	check(Src.Extent == Extent);
	const TMeshAttributeArrayBase<AttributeType>& SrcArray = static_cast<const BoundedArrayType*>(Src.ArrayPtr)->GetArrayForChannel(SrcChannel);
	TMeshAttributeArrayBase<AttributeType>& DestArray = static_cast<BoundedArrayType*>(ArrayPtr)->GetArrayForChannel(DestChannel);
	const int32 Num = FMath::Min(SrcArray.Num(), DestArray.Num());
	for (int32 Index = 0; Index < Num; Index++)
	{
		for (uint32 Count = 0; Count < Extent; Count++)
		{
			DestArray.GetElementBase(Index)[Count] = SrcArray.GetElementBase(Index)[Count];
		}
	}
}


template <typename ElementIDType, typename AttributeType>
class TMeshAttributesRef<ElementIDType, TArrayAttribute<AttributeType>>
{
	template <typename T, typename U> friend class TMeshAttributesRef;

public:
	using BaseArrayType = TCopyQualifiersFromTo_T<AttributeType, FMeshAttributeArraySetBase>;
	using ArrayType = TCopyQualifiersFromTo_T<AttributeType, TMeshUnboundedAttributeArraySet<std::remove_cv_t<AttributeType>>>;

	/** Constructor taking a pointer to a TMeshUnboundedAttributeArraySet */
	explicit TMeshAttributesRef(BaseArrayType* InArrayPtr = nullptr, uint32 InExtent = 0)
		: ArrayPtr(InArrayPtr)
	{}

	/** Implicitly construct a TMeshAttributesRef-to-const from a regular one */
	template <typename SrcAttributeType,
			  typename DestAttributeType = AttributeType,
			  typename TEnableIf<std::is_const_v<DestAttributeType>, int>::Type = 0,
			  typename TEnableIf<!std::is_const_v<SrcAttributeType>, int>::Type = 0>
	TMeshAttributesRef(const TMeshAttributesRef<ElementIDType, TArrayAttribute<SrcAttributeType>>& InRef)
		: ArrayPtr(InRef.ArrayPtr)
	{}

	/** Implicitly construct a TMeshAttributesRef from a TMeshAttributesArray **/
	template <typename IDType = ElementIDType,
			  typename TEnableIf<!std::is_same_v<IDType, int32>, int>::Type = 0>
	TMeshAttributesRef(const TMeshAttributesRef<int32, TArrayAttribute<AttributeType>>& InRef)
		: ArrayPtr(InRef.ArrayPtr)
	{}

	/** Implicitly construct a TMeshAttributesRef-to-const from a TMeshAttributesArray */
	template <typename SrcAttributeType,
			  typename DestAttributeType = AttributeType,
			  typename IDType = ElementIDType,
			  typename TEnableIf<!std::is_same_v<IDType, int32>, int>::Type = 0,
			  typename TEnableIf<std::is_const_v<DestAttributeType>, int>::Type = 0,
			  typename TEnableIf<!std::is_const_v<SrcAttributeType>, int>::Type = 0>
	TMeshAttributesRef(const TMeshAttributesRef<int32, TArrayAttribute<SrcAttributeType>>& InRef)
		: ArrayPtr(InRef.ArrayPtr)
	{}


	/** Access elements from attribute channel 0 */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	TArrayAttribute<AttributeType> operator[](const ElementIDType ElementID) const
	{
		return TArrayAttribute<AttributeType>(static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(0), ElementID.GetValue());
	}

	/** Get the element with the given ID and channel */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	TArrayAttribute<AttributeType> Get(const ElementIDType ElementID, const int32 Channel = 0) const
	{
		return TArrayAttribute<AttributeType>(static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(Channel), ElementID.GetValue());
	}

	TArrayAttribute<AttributeType> operator[](int32 ElementIndex) const
	{
		return TArrayAttribute<AttributeType>(static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(0), ElementIndex);
	}

	TArrayAttribute<AttributeType> Get(int32 ElementIndex, const int32 Channel = 0) const
	{
		return TArrayAttribute<AttributeType>(static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(Channel), ElementIndex);
	}

	TArrayView<AttributeType> GetArrayView(int32 ElementIndex, const int32 Channel = 0) const
	{
		return Get(ElementIndex, Channel).ToArrayView();
	}

	/** In this specialization, GetRawArray returns a pointer to the attribute array container holding the attributes and their index pointers */
	const TAttributeArrayContainer<AttributeType>* GetRawArray(const int32 AttributeChannel = 0) const
	{
		if (ArrayPtr == nullptr || GetNumElements() == 0)
		{
			return nullptr;
		}

		return &static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(AttributeChannel);
	}

	/** Return whether the reference is valid or not */
	bool IsValid() const { return (ArrayPtr != nullptr); }

	/** Return default value for this attribute type */
	AttributeType GetDefaultValue() const { return static_cast<ArrayType*>(ArrayPtr)->GetDefaultValue(); }

	/** Return number of channels this attribute has */
	int32 GetNumChannels() const
	{
		return static_cast<ArrayType*>(ArrayPtr)->ArrayType::GetNumChannels();	// note: override virtual dispatch
	}

	/** Get the number of elements in this attribute array */
	int32 GetNumElements() const
	{
		return ArrayPtr->GetNumElements();
	}

	/** Get the flags for this attribute array set */
	EMeshAttributeFlags GetFlags() const { return ArrayPtr->GetFlags(); }

	/** Set the element with the given ID and index 0 to the provided value */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	void Set(const ElementIDType ElementID, TArrayAttribute<const AttributeType> Value) const
	{
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(0).Set(ElementID.GetValue(), Value.ToArrayView());
	}

	/** Set the element with the given ID and channel to the provided value */
	template <typename T = ElementIDType, typename TEnableIf<TIsDerivedFrom<T, FElementID>::Value, int>::Type = 0>
	void Set(const ElementIDType ElementID, const int32 Channel, TArrayAttribute<const AttributeType> Value) const
	{
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).Set(ElementID.GetValue(), Value.ToArrayView());
	}

	void Set(int32 ElementIndex, TArrayAttribute<const AttributeType> Value) const
	{
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(0).Set(ElementIndex, Value.ToArrayView());
	}

	void Set(int32 ElementIndex, const int32 Channel, TArrayAttribute<const AttributeType> Value) const
	{
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).Set(ElementIndex, Value.ToArrayView());
	}

	void SetArrayView(int32 ElementIndex, TArrayView<const AttributeType> Value) const
	{
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(0).Set(ElementIndex, Value);
	}

	void SetArrayView(int32 ElementIndex, const int32 Channel, TArrayView<const AttributeType> Value) const
	{
		static_cast<ArrayType*>(ArrayPtr)->GetArrayForChannel(Channel).Set(ElementIndex, Value);
	}

	/** Copies the given attribute array and index to this index */
	void Copy(TMeshAttributesConstRef<ElementIDType, TArrayAttribute<AttributeType>> Src, const int32 DestChannel = 0, const int32 SrcChannel = 0);

	/** Sets number of channels this attribute has */
	void SetNumChannels(const int32 NumChannels) const
	{
		static_cast<ArrayType*>(ArrayPtr)->ArrayType::SetNumChannels(NumChannels);	// note: override virtual dispatch
	}

	/** Inserts an attribute channel */
	void InsertChannel(const int32 Index) const
	{
		static_cast<ArrayType*>(ArrayPtr)->ArrayType::InsertChannel(Index);		// note: override virtual dispatch
	}

	/** Removes an attribute channel */
	void RemoveChannel(const int32 Index) const
	{
		static_cast<ArrayType*>(ArrayPtr)->ArrayType::RemoveChannel(Index);		// note: override virtual dispatch
	}

private:
	BaseArrayType* ArrayPtr;
};


/**
 * This is a wrapper for an allocated attributes array.
 * It holds a TUniquePtr pointing to the actual attributes array, and performs polymorphic copy and assignment,
 * as per the actual array type.
 */
class FAttributesSetEntry
{
public:
	/**
	 * Default constructor.
	 * This breaks the invariant that Ptr be always valid, but is necessary so that it can be the value type of a TMap.
	 */
	FAttributesSetEntry() = default;

	/**
	 * Construct a valid FAttributesSetEntry of the concrete type specified.
	 */
	template <typename AttributeType>
	FAttributesSetEntry(const int32 NumberOfChannels, const AttributeType& Default, const EMeshAttributeFlags Flags, const int32 NumElements, const int32 Extent)
	{
		if (Extent > 0)
		{
			Ptr = MakeUnique<TMeshAttributeArraySet<AttributeType>>(NumberOfChannels, Default, Flags, NumElements, Extent);
		}
		else
		{
			Ptr = MakeUnique<TMeshUnboundedAttributeArraySet<AttributeType>>(NumberOfChannels, Default, Flags, NumElements);
		}
	}

	/** Default destructor */
	~FAttributesSetEntry() = default;

	/** Polymorphic copy: a new copy of Other is created */
	FAttributesSetEntry(const FAttributesSetEntry& Other)
		: Ptr(Other.Ptr ? Other.Ptr->Clone() : nullptr)
	{}

	/** Default move constructor */
	FAttributesSetEntry(FAttributesSetEntry&&) = default;

	/** Polymorphic assignment */
	FAttributesSetEntry& operator=(const FAttributesSetEntry& Other)
	{
		FAttributesSetEntry Temp(Other);
		Swap(*this, Temp);
		return *this;
	}

	/** Default move assignment */
	FAttributesSetEntry& operator=(FAttributesSetEntry&&) = default;

	/** Transparent access through the TUniquePtr */
	inline const FMeshAttributeArraySetBase* Get() const { return Ptr.Get(); }
	inline const FMeshAttributeArraySetBase* operator->() const { return Ptr.Get(); }
	inline const FMeshAttributeArraySetBase& operator*() const { return *Ptr; }
	inline FMeshAttributeArraySetBase* Get() { return Ptr.Get(); }
	inline FMeshAttributeArraySetBase* operator->() { return Ptr.Get(); }
	inline FMeshAttributeArraySetBase& operator*() { return *Ptr; }

	/** Object can be coerced to bool to indicate if it is valid */
	inline explicit operator bool() const { return Ptr.IsValid(); }
	inline bool operator!() const { return !Ptr.IsValid(); }

	/** Given a type at runtime, allocate an attribute array of that type, owned by Ptr */
	void CreateArrayOfType(const uint32 Type, const uint32 Extent);

	/** Serialization */
	friend FArchive& operator<<(FArchive& Ar, FAttributesSetEntry& Entry);

private:
	TUniquePtr<FMeshAttributeArraySetBase> Ptr;
};


/**
 * This is the container for all attributes and their arrays. It wraps a TMap, mapping from attribute name to attribute array.
 * An attribute may be of any arbitrary type; we use a mixture of polymorphism and compile-time templates to handle the different types.
 */
class FAttributesSetBase
{
public:
	/** Constructor */
	FAttributesSetBase()
		: NumElements(0)
	{}

	/**
	 * Register a new attribute name with the given type (must be a member of the AttributeTypes tuple).
	 * If the attribute name is already registered, it will update it to use the new type, number of channels and flags.
	 *
	 * Example of use:
	 *
	 *		VertexInstanceAttributes().RegisterAttribute<FVector2f>( "UV", 8 );
	 *                        . . .
	 *		TVertexInstanceAttributeArray<FVector2f>& UV0 = VertexInstanceAttributes().GetAttributes<FVector2f>( "UV", 0 );
	 *		UV0[ VertexInstanceID ] = FVector2f( 1.0f, 1.0f );
	 */

	template <typename T>
	TMeshAttributesArray<typename TMeshAttributesRegisterType<T>::RefType> RegisterAttributeInternal(
		const FName AttributeName,
		const int32 NumberOfChannels = 1,
		const typename TMeshAttributesRegisterType<T>::RealAttributeType& Default = typename TMeshAttributesRegisterType<T>::RealAttributeType(),
		const EMeshAttributeFlags Flags = EMeshAttributeFlags::None)
	{
		using AttributeType = typename TMeshAttributesRegisterType<T>::AttributeType;
		using RealAttributeType = typename TMeshAttributesRegisterType<T>::RealAttributeType;
		using RefType = typename TMeshAttributesRegisterType<T>::RefType;
		const uint32 Extent = TMeshAttributesRegisterType<T>::Extent;

		if (FAttributesSetEntry* ArraySetPtr = Map.Find(AttributeName))
		{
			if ((*ArraySetPtr)->HasType<RealAttributeType>() && (*ArraySetPtr)->GetExtent() == Extent)
			{
				(*ArraySetPtr)->SetNumChannels(NumberOfChannels);
				(*ArraySetPtr)->SetFlags(Flags);
				return TMeshAttributesArray<RefType>(ArraySetPtr->Get(), Extent);
			}
			else
			{
				Map.Remove(AttributeName);
			}
		}

		FAttributesSetEntry& Entry = Map.Emplace(AttributeName, FAttributesSetEntry(NumberOfChannels, Default, Flags, NumElements, Extent));
		return TMeshAttributesArray<RefType>(Entry.Get(), Extent);
	}

	/**
	 * Register a new simple attribute.
	 * e.g. RegisterAttribute<float>(...)
	 * 
	 * Obtain a reference to this with GetAttributesRef<float>(...)
	 */
	template <typename T,
			  typename TEnableIf<!TIsArray<T>::Value, int>::Type = 0>
	TMeshAttributesArray<typename TMeshAttributesRegisterType<T>::RefType> RegisterAttribute(
		const FName AttributeName,
		const int32 NumberOfChannels = 1,
		const T& Default = T(),
		const EMeshAttributeFlags Flags = EMeshAttributeFlags::None)
	{
		return this->RegisterAttributeInternal<T>(AttributeName, NumberOfChannels, Default, Flags);
	}

	/**
	 * Register a new fixed array attribute.
	 * e.g. RegisterAttribute<float[3]>(...)
	 * 
	 * Obtain a reference to this with GetAttributesRef<TArrayView<float>>(...)
	 */
	template <typename T,
			  typename TEnableIf<TIsArray<T>::Value, int>::Type = 0>
	TMeshAttributesArray<typename TMeshAttributesRegisterType<T>::RefType> RegisterAttribute(
		const FName AttributeName,
		const int32 NumberOfChannels = 1,
		const typename TMeshAttributesRegisterType<T>::RealAttributeType& Default = typename TMeshAttributesRegisterType<T>::RealAttributeType(),
		const EMeshAttributeFlags Flags = EMeshAttributeFlags::None)
	{
		return this->RegisterAttributeInternal<T>(AttributeName, NumberOfChannels, Default, Flags);
	}

	/**
	 * Register a new unbounded array attribute.
	 * e.g. RegisterAttribute<float[]>(...)
	 * 
	 * Obtain a reference to this with GetAttributesRef<TArrayAttribute<float>>(...)
	 */
	template <typename T,
			  typename TEnableIf<std::is_same_v<typename TMeshAttributesRegisterType<T>::RealAttributeType, int>, int>::Type = 0>
	TMeshAttributesArray<typename TMeshAttributesRegisterType<T>::RefType> RegisterIndexAttribute(
		const FName AttributeName,
		const int32 NumberOfChannels = 1,
		const EMeshAttributeFlags Flags = EMeshAttributeFlags::None)
	{
		return this->RegisterAttributeInternal<T>(AttributeName, NumberOfChannels, int32(INDEX_NONE), Flags | EMeshAttributeFlags::IndexReference);
	}

	/**
	 * Unregister an attribute with the given name.
	 */
	void UnregisterAttribute(const FName AttributeName)
	{
		Map.Remove(AttributeName);
	}

	/** Determines whether an attribute exists with the given name */
	bool HasAttribute(const FName AttributeName) const
	{
		return Map.Contains(AttributeName);
	}

	/**
	 * Determines whether an attribute of the given type exists with the given name
	 */
	template <typename T>
	bool HasAttributeOfType(const FName AttributeName) const
	{
		using RealAttributeType = typename TMeshAttributesRefType<T>::RealAttributeType;

		if (const FAttributesSetEntry* ArraySetPtr = Map.Find(AttributeName))
		{
			return (*ArraySetPtr)->HasType<RealAttributeType>() &&
				   (*ArraySetPtr)->GetExtent() >= TMeshAttributesRefType<T>::MinExpectedExtent &&
				   (*ArraySetPtr)->GetExtent() <= TMeshAttributesRefType<T>::MaxExpectedExtent;
		}

		return false;
	}

	/** Initializes all attributes to have the given number of elements with the default value */
	void Initialize(const int32 Count)
	{
		NumElements = Count;
		for (auto& MapEntry : Map)
		{
			MapEntry.Value->Initialize(Count);
		}
	}

	/** Sets all attributes to have the given number of elements, preserving existing values and filling extra elements with the default value */
	void SetNumElements(const int32 Count)
	{
		NumElements = Count;
		for (auto& MapEntry : Map)
		{
			MapEntry.Value->SetNumElements(Count);
		}
	}

	/** Gets the number of elements in the attribute set */
	int32 GetNumElements() const
	{
		return NumElements;
	}

	/** Applies the given remapping to the attributes set */
	void Remap(const TSparseArray<int32>& IndexRemap);

	/** Returns an array of all the attribute names registered */
	template <typename Allocator>
	void GetAttributeNames(TArray<FName, Allocator>& OutAttributeNames) const
	{
		Map.GetKeys(OutAttributeNames);
	}

	/** Determine whether an attribute has any of the given flags */
	bool DoesAttributeHaveAnyFlags(const FName AttributeName, EMeshAttributeFlags AttributeFlags) const
	{
		if (const FAttributesSetEntry* ArraySetPtr = Map.Find(AttributeName))
		{
			return EnumHasAnyFlags((*ArraySetPtr)->GetFlags(), AttributeFlags);
		}

		return false;
	}

	/** Determine whether an attribute has all of the given flags */
	bool DoesAttributeHaveAllFlags(const FName AttributeName, EMeshAttributeFlags AttributeFlags) const
	{
		if (const FAttributesSetEntry* ArraySetPtr = Map.Find(AttributeName))
		{
			return EnumHasAllFlags((*ArraySetPtr)->GetFlags(), AttributeFlags);
		}

		return false;
	}

	uint32 GetHash(const FName AttributeName) const
	{
		if (const FAttributesSetEntry* ArraySetPtr = Map.Find(AttributeName))
		{
			return (*ArraySetPtr)->GetHash();
		}
		return 0;
	}

	/**
	 * Insert a new element at the given index.
	 * The public API version of this function takes an ID of ElementIDType instead of a typeless index.
	 */
	void Insert(const int32 Index)
	{
		NumElements = FMath::Max(NumElements, Index + 1);

		for (auto& MapEntry : Map)
		{
			MapEntry.Value->Insert(Index);

#if DO_CHECK
			// Log out some more debug info when this fails, as it's failing in public builds with no known repro
			if (MapEntry.Value->GetNumElements() != NumElements)
			{
				UE_LOGFMT(LogMeshDescription, Error, "NumElements mismatch in FAttributesSetBase::Insert");
				UE_LOGFMT(LogMeshDescription, Error, "NumElements: {NumElements}", NumElements);
				UE_LOGFMT(LogMeshDescription, Error, "Index: {Index}", Index);
				UE_LOGFMT(LogMeshDescription, Error, "Mismatched attribute: {Attribute}\n", MapEntry.Key.ToString());

				for (const TPair<FName, FAttributesSetEntry>& Pair : Map)
				{
					UE_LOGFMT(LogMeshDescription, Error, "Attribute {Attribute} NumElements: {NumElements}\n", Pair.Key.ToString(), Pair.Value->GetNumElements());
				}

				// Add some info to the assert message anyway, in case the logs are suppressed
				checkf(false, TEXT("NumElements mismatch in FAttributesSetBase::Insert: %s: %i != %i"), *MapEntry.Key.ToString(), MapEntry.Value->GetNumElements(), NumElements);
			}
#endif
		}
	}

	/**
	 * Remove an element at the given index.
	 * The public API version of this function takes an ID of ElementIDType instead of a typeless index.
	 */
	void Remove(const int32 Index)
	{
		for (auto& MapEntry : Map)
		{
			MapEntry.Value->Remove(Index);
		}
	}

	/**
	 * Get an attribute array with the given type and name.
	 * The attribute type must correspond to the type passed as the template parameter.
	 */
	template <typename T>
	TMeshAttributesConstRef<int32, typename TMeshAttributesRefType<T>::ConstRefType> GetAttributesRef(const FName AttributeName) const
	{
		using RefType = typename TMeshAttributesRefType<T>::ConstRefType;

		if (const FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			using AttributeType = typename TMeshAttributesRefType<T>::AttributeType;
			using RealAttributeType = typename TMeshAttributesRefType<T>::RealAttributeType;

			if ((*ArraySetPtr)->HasType<RealAttributeType>())
			{
				uint32 ActualExtent = (*ArraySetPtr)->GetExtent();
				if (ActualExtent >= TMeshAttributesRefType<T>::MinExpectedExtent && ActualExtent <= TMeshAttributesRefType<T>::MaxExpectedExtent)
				{
					return TMeshAttributesConstRef<int32, RefType>(ArraySetPtr->Get(), ActualExtent);
				}
			}
		}

		return TMeshAttributesConstRef<int32, RefType>();
	}

	template <typename T>
	TMeshAttributesRef<int32, typename TMeshAttributesRefType<T>::RefType> GetAttributesRef(const FName AttributeName)
	{
		using RefType = typename TMeshAttributesRefType<T>::RefType;

		if (FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			using AttributeType = typename TMeshAttributesRefType<T>::AttributeType;
			using RealAttributeType = typename TMeshAttributesRefType<T>::RealAttributeType;

			if ((*ArraySetPtr)->HasType<RealAttributeType>())
			{
				uint32 ActualExtent = (*ArraySetPtr)->GetExtent();
				if (ActualExtent >= TMeshAttributesRefType<T>::MinExpectedExtent && ActualExtent <= TMeshAttributesRefType<T>::MaxExpectedExtent)
				{
					return TMeshAttributesRef<int32, RefType>(ArraySetPtr->Get(), ActualExtent);
				}
			}
		}

		return TMeshAttributesRef<int32, RefType>();
	}

	void AppendAttributesFrom(const FAttributesSetBase& OtherAttributesSet);

protected:
	/** Serialization */
	friend MESHDESCRIPTION_API FArchive& operator<<(FArchive& Ar, FAttributesSetBase& AttributesSet);

	template <typename T>
	friend void SerializeLegacy(FArchive& Ar, FAttributesSetBase& AttributesSet);

	/** The actual container */
	TMap<FName, FAttributesSetEntry> Map;

	/** The number of elements in each attribute array */
	int32 NumElements;
};


/**
 * This is a version of the attributes set container which accesses elements by typesafe IDs.
 * This prevents access of (for example) vertex instance attributes by vertex IDs.
 */
template <typename ElementIDType>
class TAttributesSet final : public FAttributesSetBase
{
	using FAttributesSetBase::Insert;
	using FAttributesSetBase::Remove;

public:
	/**
	 * Get an attribute array with the given type and name.
	 * The attribute type must correspond to the type passed as the template parameter.
	 *
	 * Example of use:
	 *
	 *		TVertexAttributesConstRef<FVector> VertexPositions = VertexAttributes().GetAttributesRef<FVector>( "Position" ); // note: assign to value type
	 *		for( const FVertexID VertexID : GetVertices().GetElementIDs() )
	 *		{
	 *			const FVector Position = VertexPositions.Get( VertexID );
	 *			DoSomethingWith( Position );
	 *		}
	 *
	 * Note that the returned object is a value type which should be assigned and passed by value, not reference.
	 * It is valid for as long as this TAttributesSet object exists.
	 */
	template <typename T>
	TMeshAttributesConstRef<ElementIDType, typename TMeshAttributesRefType<T>::ConstRefType> GetAttributesRef(const FName AttributeName) const
	{
		using RefType = typename TMeshAttributesRefType<T>::ConstRefType;

		if (const FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			using AttributeType = typename TMeshAttributesRefType<T>::AttributeType;
			using RealAttributeType = typename TMeshAttributesRefType<T>::RealAttributeType;

			if ((*ArraySetPtr)->HasType<RealAttributeType>())
			{
				uint32 ActualExtent = (*ArraySetPtr)->GetExtent();
				if (ActualExtent >= TMeshAttributesRefType<T>::MinExpectedExtent && ActualExtent <= TMeshAttributesRefType<T>::MaxExpectedExtent)
				{
					return TMeshAttributesConstRef<ElementIDType, RefType>(ArraySetPtr->Get(), ActualExtent);
				}
			}
		}

		return TMeshAttributesConstRef<ElementIDType, RefType>();
	}

	// Non-const version
	template <typename T>
	TMeshAttributesRef<ElementIDType, typename TMeshAttributesRefType<T>::RefType> GetAttributesRef(const FName AttributeName)
	{
		using RefType = typename TMeshAttributesRefType<T>::RefType;

		if (FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			using AttributeType = typename TMeshAttributesRefType<T>::AttributeType;
			using RealAttributeType = typename TMeshAttributesRefType<T>::RealAttributeType;

			if ((*ArraySetPtr)->HasType<RealAttributeType>())
			{
				uint32 ActualExtent = (*ArraySetPtr)->GetExtent();
				if (ActualExtent >= TMeshAttributesRefType<T>::MinExpectedExtent && ActualExtent <= TMeshAttributesRefType<T>::MaxExpectedExtent)
				{
					return TMeshAttributesRef<ElementIDType, RefType>(ArraySetPtr->Get(), ActualExtent);
				}
			}
		}

		return TMeshAttributesRef<ElementIDType, RefType>();
	}


	UE_DEPRECATED(5.0, "Please use GetAttributeChannelCount() instead.")
	int32 GetAttributeIndexCount(const FName AttributeName) const
	{
		return GetAttributeChannelCount(AttributeName);
	}

	/** Returns the number of indices for the attribute with the given name */
	int32 GetAttributeChannelCount(const FName AttributeName) const
	{
		if (const FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			return (*ArraySetPtr)->GetNumChannels();
		}

		return 0;
	}

	template <typename AttributeType>
	UE_DEPRECATED(5.0, "Please use GetAttributeChannelCount() instead.")
	int32 GetAttributeIndexCount(const FName AttributeName) const
	{
		if (const FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			if ((*ArraySetPtr)->HasType<AttributeType>())
			{
				using ArrayType = TMeshAttributeArraySet<AttributeType>;
				return static_cast<const ArrayType*>( ArraySetPtr->Get() )->ArrayType::GetNumChannels();	// note: override virtual dispatch
			}
		}

		return 0;
	}

	UE_DEPRECATED(5.0, "Please use SetAttributeChannelCount() instead.")
	void SetAttributeIndexCount(const FName AttributeName, const int32 NumChannels)
	{
		SetAttributeChannelCount(AttributeName, NumChannels);
	}

	/** Sets the number of indices for the attribute with the given name */
	void SetAttributeChannelCount(const FName AttributeName, const int32 NumChannels)
	{
		if (FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			(*ArraySetPtr)->SetNumChannels(NumChannels);
		}
	}

	template <typename AttributeType>
	UE_DEPRECATED(5.0, "Please use untemplated SetAttributeChannelCount() instead.")
	void SetAttributeIndexCount(const FName AttributeName, const int32 NumIndices)
	{
		if (FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			if ((*ArraySetPtr)->HasType<AttributeType>())
			{
				using ArrayType = TMeshAttributeArraySet<AttributeType>;
				static_cast<ArrayType*>(ArraySetPtr->Get())->ArrayType::SetNumChannels(NumIndices);	// note: override virtual dispatch
			}
		}
	}

	UE_DEPRECATED(5.0, "Please use InsertAttributeChannel() instead.")
	void InsertAttributeIndex(const FName AttributeName, const int32 Index)
	{
		InsertAttributeChannel(AttributeName, Index);
	}

	/** Insert a new index for the attribute with the given name */
	void InsertAttributeChannel(const FName AttributeName, const int32 Index)
	{
		if (FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			(*ArraySetPtr)->InsertChannel(Index);
		}
	}

	template <typename AttributeType>
	UE_DEPRECATED(5.0, "Please use untemplated InsertAttributeIndexCount() instead.")
	void InsertAttributeIndex(const FName AttributeName, const int32 Index)
	{
		if (FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			if ((*ArraySetPtr)->HasType<AttributeType>())
			{
				using ArrayType = TMeshAttributeArraySet<AttributeType>;
				static_cast<ArrayType*>(ArraySetPtr->Get())->ArrayType::InsertChannel(Index);	// note: override virtual dispatch
			}
		}
	}

	UE_DEPRECATED(5.0, "Please use RemoveAttributeChannel() instead.")
	void RemoveAttributeIndex(const FName AttributeName, const int32 Index)
	{
		RemoveAttributeChannel(AttributeName, Index);
	}

	/** Remove an existing index from the attribute with the given name */
	void RemoveAttributeChannel(const FName AttributeName, const int32 Index)
	{
		if (FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			(*ArraySetPtr)->RemoveChannel(Index);
		}
	}

	template <typename AttributeType>
	UE_DEPRECATED(5.0, "Please use untemplated RemoveAttributeIndexCount() instead.")
	void RemoveAttributeIndex(const FName AttributeName, const int32 Index)
	{
		if (FAttributesSetEntry* ArraySetPtr = this->Map.Find(AttributeName))
		{
			if ((*ArraySetPtr)->HasType<AttributeType>())
			{
				using ArrayType = TMeshAttributeArraySet<AttributeType>;
				static_cast<ArrayType*>(ArraySetPtr->Get())->ArrayType::RemoveChannel(Index);	// note: override virtual dispatch
			}
		}
	}

	/**
	 * Get an attribute value for the given element ID.
	 * Note: it is generally preferable to get a TMeshAttributesRef and access elements through that, if you wish to access more than one.
	 */
	template <typename T>
	T GetAttribute(const ElementIDType ElementID, const FName AttributeName, const int32 AttributeChannel = 0) const
	{
		using RefType = typename TMeshAttributesRefType<T>::ConstRefType;
		using AttributeType = typename TMeshAttributesRefType<T>::AttributeType;
		using RealAttributeType = typename TMeshAttributesRefType<T>::RealAttributeType;

		const FMeshAttributeArraySetBase* ArraySetPtr = this->Map.FindChecked(AttributeName).Get();
		uint32 ActualExtent = ArraySetPtr->GetExtent();
		check(ArraySetPtr->HasType<RealAttributeType>());
		check(ActualExtent >= TMeshAttributesRefType<AttributeType>::MinExpectedExtent && ActualExtent <= TMeshAttributesRefType<AttributeType>::MaxExpectedExtent);

		TMeshAttributesConstRef<FElementID, RefType> Ref(ArraySetPtr, ActualExtent);
		return Ref.Get(ElementID, AttributeChannel);
	}


	/**
	 * Set an attribute value for the given element ID.
	 * Note: it is generally preferable to get a TMeshAttributesRef and set multiple elements through that.
	 */
	template <typename T>
	void SetAttribute(const ElementIDType ElementID, const FName AttributeName, const int32 AttributeChannel, const T& AttributeValue)
	{
		using NonConstRefType = typename TMeshAttributesRefType<T>::NonConstRefType;
		using AttributeType = typename TMeshAttributesRefType<T>::AttributeType;
		using RealAttributeType = typename TMeshAttributesRefType<T>::RealAttributeType;

		FMeshAttributeArraySetBase* ArraySetPtr = this->Map.FindChecked(AttributeName).Get();
		uint32 ActualExtent = ArraySetPtr->GetExtent();
		check(ArraySetPtr->HasType<std::remove_cv_t<RealAttributeType>>());
		check(ActualExtent >= TMeshAttributesRefType<AttributeType>::MinExpectedExtent && ActualExtent <= TMeshAttributesRefType<AttributeType>::MaxExpectedExtent);

		TMeshAttributesRef<FElementID, NonConstRefType> Ref(ArraySetPtr, ActualExtent);
		return Ref.Set(ElementID, AttributeChannel, AttributeValue);
	}



	/** Inserts a default-initialized value for all attributes of the given ID */
	inline void Insert(const ElementIDType ElementID)
	{
		this->Insert(ElementID.GetValue());
	}

	/** Removes all attributes with the given ID */
	inline void Remove(const ElementIDType ElementID)
	{
		this->Remove(ElementID.GetValue());
	}

	/**
	 * Call the supplied function on each attribute.
	 * The prototype should be Func( const FName AttributeName, auto AttributesRef );
	 */
	template <typename ForEachFunc> void ForEach(ForEachFunc Func);

	/**
	* Call the supplied function on each attribute.
	* The prototype should be Func( const FName AttributeName, auto AttributesConstRef );
	*/
	template <typename ForEachFunc> void ForEach(ForEachFunc Func) const;

	/**
	* Call the supplied function on each attribute that matches the given type.
	* The type can be given as either a plain type T, TArrayView<T> or TArrayAttribute<T>. 
	* The prototype should be Func( const FName AttributeName, auto AttributesRef );
	*/
	template <typename AttributeType, typename ForEachFunc> void ForEachByType(ForEachFunc Func);
	
	/**
	* Call the supplied function on each attribute that matches the given type.
	* The type can be given as either a plain type T, TArrayView<T> or TArrayAttribute<T>. 
	* The prototype should be Func( const FName AttributeName, auto AttributesConstRef );
	*/
	template <typename AttributeType, typename ForEachFunc> void ForEachByType(ForEachFunc Func) const;
};


/**
 * We need a mechanism by which we can iterate all items in the attribute map and perform an arbitrary operation on each.
 * We require polymorphic behavior, as attribute arrays are templated on their attribute type, and derived from a generic base class.
 * However, we cannot have a virtual templated method, so we use a different approach.
 *
 * Effectively, we wish to cast the attribute array depending on the type member of the base class as we iterate through the map.
 * This might look something like this:
 *
 *    template <typename FuncType>
 *    void ForEach(FuncType Func)
 *    {
 *        for (const auto& MapEntry : Map)
 *        {
 *            const uint32 Type = MapEntry.Value->GetType();
 *            switch (Type)
 *            {
 *                case 0: Func(static_cast<TMeshAttributeArraySet<FVector3f>*>(MapEntry.Value.Get()); break;
 *                case 1: Func(static_cast<TMeshAttributeArraySet<FVector4f>*>(MapEntry.Value.Get()); break;
 *                case 2: Func(static_cast<TMeshAttributeArraySet<FVector2f>*>(MapEntry.Value.Get()); break;
 *                case 3: Func(static_cast<TMeshAttributeArraySet<float>*>(MapEntry.Value.Get()); break;
 *                      ....
 *            }
 *        }
 *    }
 *
 * (The hope is that the compiler would optimize the switch into a jump table so we get O(1) dispatch even as the number of attribute types
 * increases.)
 *
 * The approach taken here is to generate a jump table at compile time, one entry per possible attribute type.
 * The function Dispatch(...) is the actual function which gets called.
 * MakeJumpTable() is the constexpr function which creates a static jump table at compile time.
 */


/**
 * Class which implements a function jump table to be automatically generated at compile time.
 * This is used by TAttributesSet to provide O(1) dispatch by attribute type at runtime.
 */
template <typename FnType, uint32 Size>
struct TJumpTable
{
	template <typename... T>
	explicit constexpr TJumpTable( T... Ts ) : Fns{ Ts... } {}

	FnType* Fns[Size];
};



namespace ForEachImpl
{
	// Declare type of jump table used to dispatch functions
	template <typename ElementIDType, typename ForEachFunc>
	using JumpTableType = TJumpTable<void(FName, ForEachFunc, FMeshAttributeArraySetBase*), TTupleArity<AttributeTypes>::Value>;

	// Define dispatch function
	template <typename ElementIDType, typename ForEachFunc, uint32 I>
	static void Dispatch(FName Name, ForEachFunc Fn, FMeshAttributeArraySetBase* Attributes)
	{
		using AttributeType = typename TTupleElement<I, AttributeTypes>::Type;
		if (Attributes->GetExtent() == 0)
		{
			Fn(Name, TMeshAttributesRef<ElementIDType, TArrayAttribute<AttributeType>>(static_cast<TMeshUnboundedAttributeArraySet<AttributeType>*>(Attributes)));
		}
		else if (Attributes->GetExtent() == 1)
		{
			Fn(Name, TMeshAttributesRef<ElementIDType, AttributeType>(static_cast<TMeshAttributeArraySet<AttributeType>*>(Attributes)));
		}
		else
		{
			Fn(Name, TMeshAttributesRef<ElementIDType, TArrayView<AttributeType>>(static_cast<TMeshAttributeArraySet<AttributeType>*>(Attributes), Attributes->GetExtent()));
		}
	}

	// Build ForEach jump table at compile time, a separate instantiation of Dispatch for each attribute type
	template <typename ElementIDType, typename ForEachFunc, uint32... Is>
	static constexpr JumpTableType<ElementIDType, ForEachFunc> MakeJumpTable(TIntegerSequence< uint32, Is...>)
	{
		return JumpTableType<ElementIDType, ForEachFunc>(Dispatch<ElementIDType, ForEachFunc, Is>...);
	}
}

template <typename ElementIDType>
template <typename ForEachFunc>
void TAttributesSet<ElementIDType>::ForEach(ForEachFunc Func)
{
	// Construct compile-time jump table for dispatching ForEachImpl::Dispatch() by the attribute type at runtime
	static constexpr ForEachImpl::JumpTableType<ElementIDType, ForEachFunc>
		JumpTable = ForEachImpl::MakeJumpTable<ElementIDType, ForEachFunc>(TMakeIntegerSequence<uint32, TTupleArity<AttributeTypes>::Value>());

	for (auto& MapEntry : this->Map)
	{
		const uint32 Type = MapEntry.Value->GetType();
		JumpTable.Fns[Type](MapEntry.Key, Func, MapEntry.Value.Get());
	}
}


namespace ForEachConstImpl
{
	// Declare type of jump table used to dispatch functions
	template <typename ElementIDType, typename ForEachFunc>
	using JumpTableType = TJumpTable<void(FName, ForEachFunc, const FMeshAttributeArraySetBase*), TTupleArity<AttributeTypes>::Value>;

	// Define dispatch function
	template <typename ElementIDType, typename ForEachFunc, uint32 I>
	static void Dispatch(FName Name, ForEachFunc Fn, const FMeshAttributeArraySetBase* Attributes)
	{
		using AttributeType = typename TTupleElement<I, AttributeTypes>::Type;
		if (Attributes->GetExtent() == 0)
		{
			Fn(Name, TMeshAttributesConstRef<ElementIDType, TArrayAttribute<const AttributeType>>(static_cast<const TMeshUnboundedAttributeArraySet<AttributeType>*>(Attributes)));
		}
		else if (Attributes->GetExtent() == 1)
		{
			Fn(Name, TMeshAttributesConstRef<ElementIDType, AttributeType>(static_cast<const TMeshAttributeArraySet<AttributeType>*>(Attributes)));
		}
		else
		{
			Fn(Name, TMeshAttributesConstRef<ElementIDType, TArrayView<const AttributeType>>(static_cast<const TMeshAttributeArraySet<AttributeType>*>(Attributes), Attributes->GetExtent()));
		}
	}

	// Build ForEach jump table at compile time, a separate instantiation of Dispatch for each attribute type
	template <typename ElementIDType, typename ForEachFunc, uint32... Is>
	static constexpr JumpTableType<ElementIDType, ForEachFunc> MakeJumpTable(TIntegerSequence< uint32, Is...>)
	{
		return JumpTableType<ElementIDType, ForEachFunc>(Dispatch<ElementIDType, ForEachFunc, Is>...);
	}
}

template <typename ElementIDType>
template <typename ForEachFunc>
void TAttributesSet<ElementIDType>::ForEach(ForEachFunc Func) const
{
	// Construct compile-time jump table for dispatching ForEachImpl::Dispatch() by the attribute type at runtime
	static constexpr ForEachConstImpl::JumpTableType<ElementIDType, ForEachFunc>
		JumpTable = ForEachConstImpl::MakeJumpTable<ElementIDType, ForEachFunc>(TMakeIntegerSequence<uint32, TTupleArity<AttributeTypes>::Value>());

	for (const auto& MapEntry : this->Map)
	{
		const uint32 Type = MapEntry.Value->GetType();
		JumpTable.Fns[Type](MapEntry.Key, Func, MapEntry.Value.Get());
	}
}


namespace ForEachByTypeImpl
{
	template<typename ElementIDType, typename AttributeType, typename ForEachFunc>
    struct DispatchFunctor
	{
		void operator()(FName Name, ForEachFunc Fn, FMeshAttributeArraySetBase* Attributes)
		{
			if (TTupleIndex<AttributeType, AttributeTypes>::Value == Attributes->GetType() && Attributes->GetExtent() == 1)
			{
				Fn(Name, TMeshAttributesRef<ElementIDType, AttributeType>(static_cast<TMeshAttributeArraySet<AttributeType>*>(Attributes)));
			}
		}
	};

	template<typename ElementIDType,  typename AttributeType, typename ForEachFunc>
    struct DispatchFunctor<ElementIDType, TArrayView<AttributeType>, ForEachFunc>
	{
		void operator()(FName Name, ForEachFunc Fn, FMeshAttributeArraySetBase* Attributes)
		{
			if (TTupleIndex<AttributeType, AttributeTypes>::Value == Attributes->GetType() && Attributes->GetExtent() >= 1)
			{
				Fn(Name, TMeshAttributesRef<ElementIDType, TArrayView<AttributeType>>(static_cast<TMeshAttributeArraySet<AttributeType>*>(Attributes), Attributes->GetExtent()));
			}
		}
	};

	template<typename ElementIDType, typename AttributeType, typename ForEachFunc>
    struct DispatchFunctor<ElementIDType, TArrayAttribute<AttributeType>, ForEachFunc>
	{
		void operator()(FName Name, ForEachFunc Fn, FMeshAttributeArraySetBase* Attributes)
		{
			if (TTupleIndex<AttributeType, AttributeTypes>::Value == Attributes->GetType() && Attributes->GetExtent() == 0)
			{
				Fn(Name, TMeshAttributesRef<ElementIDType, TArrayAttribute<AttributeType>>(static_cast<TMeshUnboundedAttributeArraySet<AttributeType>*>(Attributes)));
			}
		}
	};
	
	template<typename ElementIDType, typename AttributeType, typename ForEachFunc>
    struct ConstDispatchFunctor
	{
		void operator()(FName Name, ForEachFunc Fn, const FMeshAttributeArraySetBase* Attributes)
		{
			if (TTupleIndex<AttributeType, AttributeTypes>::Value == Attributes->GetType() && Attributes->GetExtent() == 1)
			{
				Fn(Name, TMeshAttributesConstRef<ElementIDType, AttributeType>(static_cast<const TMeshAttributeArraySet<AttributeType>*>(Attributes)));
			}
		}
	};

	template<typename ElementIDType,  typename AttributeType, typename ForEachFunc>
    struct ConstDispatchFunctor<ElementIDType, TArrayView<AttributeType>, ForEachFunc>
	{
		void operator()(FName Name, ForEachFunc Fn, const FMeshAttributeArraySetBase* Attributes)
		{
			if (TTupleIndex<AttributeType, AttributeTypes>::Value == Attributes->GetType() && Attributes->GetExtent() >= 1)
			{
				Fn(Name, TMeshAttributesConstRef<ElementIDType, TArrayView<const AttributeType>>(static_cast<const TMeshAttributeArraySet<AttributeType>*>(Attributes), Attributes->GetExtent()));
			}
		}
	};

	template<typename ElementIDType, typename AttributeType, typename ForEachFunc>
    struct ConstDispatchFunctor<ElementIDType, TArrayAttribute<AttributeType>, ForEachFunc>
	{
		void operator()(FName Name, ForEachFunc Fn, const FMeshAttributeArraySetBase* Attributes)
		{
			if (TTupleIndex<AttributeType, AttributeTypes>::Value == Attributes->GetType() && Attributes->GetExtent() == 0)
			{
				Fn(Name, TMeshAttributesConstRef<ElementIDType, TArrayAttribute<const AttributeType>>(static_cast<const TMeshUnboundedAttributeArraySet<AttributeType>*>(Attributes)));
			}
		}
	};}

template <typename ElementIDType>
template < typename AttributeType, typename ForEachFunc>
void TAttributesSet<ElementIDType>::ForEachByType(ForEachFunc Func)
{
	for (auto& MapEntry : this->Map)
	{
		ForEachByTypeImpl::DispatchFunctor<ElementIDType, std::remove_const_t<AttributeType>, ForEachFunc>()(MapEntry.Key, Func, MapEntry.Value.Get());
	}
}

template <typename ElementIDType>
template < typename AttributeType, typename ForEachFunc>
void TAttributesSet<ElementIDType>::ForEachByType(ForEachFunc Func) const
{
	for (const auto& MapEntry : this->Map)
	{
		ForEachByTypeImpl::ConstDispatchFunctor<ElementIDType, std::remove_const_t<AttributeType>, ForEachFunc>()(MapEntry.Key, Func, MapEntry.Value.Get());
	}
}


/**
 * This is a similar approach to ForEach, above.
 * Given a type index, at runtime, we wish to create an attribute array of the corresponding type; essentially a factory.
 *
 * We generate a jump table at compile time, containing generated functions to register attributes of each type.
 */
namespace CreateTypeImpl
{
	// Declare type of jump table used to dispatch functions
	using JumpTableType = TJumpTable<TUniquePtr<FMeshAttributeArraySetBase>(uint32), TTupleArity<AttributeTypes>::Value>;

	// Define dispatch function
	template <uint32 I>
	static TUniquePtr<FMeshAttributeArraySetBase> Dispatch(uint32 Extent)
	{
		using AttributeType = typename TTupleElement<I, AttributeTypes>::Type;
		if (Extent > 0)
		{
			return MakeUnique<TMeshAttributeArraySet<AttributeType>>(Extent);
		}
		else
		{
			return MakeUnique<TMeshUnboundedAttributeArraySet<AttributeType>>();
		}
	}

	// Build RegisterAttributeOfType jump table at compile time, a separate instantiation of Dispatch for each attribute type
	template <uint32... Is>
	static constexpr JumpTableType MakeJumpTable(TIntegerSequence< uint32, Is...>)
	{
		return JumpTableType(Dispatch<Is>...);
	}
}

inline void FAttributesSetEntry::CreateArrayOfType(const uint32 Type, const uint32 Extent)
{
	static constexpr CreateTypeImpl::JumpTableType JumpTable = CreateTypeImpl::MakeJumpTable(TMakeIntegerSequence<uint32, TTupleArity<AttributeTypes>::Value>());
	Ptr = JumpTable.Fns[Type](Extent);
}

