// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "MeshAttributeArray.h"
#include "MeshDescription.h"
#include "UObject/NameTypes.h"

struct FEdgeID;
struct FPolygonGroupID;
struct FUVID;
struct FVertexID;
struct FVertexInstanceID;

namespace MeshAttribute
{
	namespace Vertex
	{
		extern MESHDESCRIPTION_API const FName Position;
	}

	namespace VertexInstance
	{
		extern MESHDESCRIPTION_API const FName VertexIndex;
	}

	namespace Edge
	{
		extern MESHDESCRIPTION_API const FName VertexIndex;
	}

	namespace Triangle
	{
		extern MESHDESCRIPTION_API const FName VertexInstanceIndex;
		extern MESHDESCRIPTION_API const FName PolygonIndex;
		extern MESHDESCRIPTION_API const FName EdgeIndex;
		extern MESHDESCRIPTION_API const FName VertexIndex;
		extern MESHDESCRIPTION_API const FName UVIndex;
		extern MESHDESCRIPTION_API const FName PolygonGroupIndex;
	}

	namespace UV
	{
		extern MESHDESCRIPTION_API const FName UVCoordinate;
	}

	namespace Polygon
	{
		extern MESHDESCRIPTION_API const FName PolygonGroupIndex;
	}
}


class FMeshAttributes
{
public:
	explicit FMeshAttributes(FMeshDescription& InMeshDescription)
		: MeshDescription(InMeshDescription)
	{}

	virtual ~FMeshAttributes() = default;

	MESHDESCRIPTION_API virtual void Register(bool bKeepExistingAttribute = false);

	static bool IsReservedAttributeName(const FName InAttributeName)
	{
		return InAttributeName == MeshAttribute::Vertex::Position ||
			   InAttributeName == MeshAttribute::VertexInstance::VertexIndex ||
			   InAttributeName == MeshAttribute::Edge::VertexIndex ||
			   InAttributeName == MeshAttribute::Triangle::VertexInstanceIndex ||
			   InAttributeName == MeshAttribute::Triangle::PolygonIndex ||
			   InAttributeName == MeshAttribute::Triangle::EdgeIndex ||
			   InAttributeName == MeshAttribute::Triangle::VertexIndex ||
			   InAttributeName == MeshAttribute::Triangle::UVIndex ||
			   InAttributeName == MeshAttribute::Triangle::PolygonGroupIndex ||
			   InAttributeName == MeshAttribute::UV::UVCoordinate ||
			   InAttributeName == MeshAttribute::Polygon::PolygonGroupIndex;
	}

	/** Accessors for cached vertex position array */
	TVertexAttributesRef<FVector3f> GetVertexPositions()
	{
		return MeshDescription.VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
	}

	TVertexAttributesConstRef<FVector3f> GetVertexPositions() const
	{
		return MeshDescription.VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
	}

	/** Accessors for array of vertex IDs for vertex instances */
	TVertexInstanceAttributesRef<FVertexID> GetVertexInstanceVertexIndices()
	{
		return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVertexID>(MeshAttribute::VertexInstance::VertexIndex);
	}

	TVertexInstanceAttributesConstRef<FVertexID> GetVertexInstanceVertexIndices() const
	{
		return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVertexID>(MeshAttribute::VertexInstance::VertexIndex);
	}

	/** Accessors for array of vertex IDs for edges */
	TEdgeAttributesRef<TArrayView<FVertexID>> GetEdgeVertexIndices()
	{
		return MeshDescription.EdgeAttributes().GetAttributesRef<TArrayView<FVertexID>>(MeshAttribute::Edge::VertexIndex);
	}
	
	TEdgeAttributesConstRef<TArrayView<FVertexID>> GetEdgeVertexIndices() const
	{
		return MeshDescription.EdgeAttributes().GetAttributesRef<TArrayView<FVertexID>>(MeshAttribute::Edge::VertexIndex);
	}

	/** Accessors for array of vertex instance IDs for triangles */
	TTriangleAttributesRef<TArrayView<FVertexInstanceID>> GetTriangleVertexInstanceIndices()
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FVertexInstanceID>>(MeshAttribute::Triangle::VertexInstanceIndex);
	}

	TTriangleAttributesConstRef<TArrayView<FVertexInstanceID>> GetTriangleVertexInstanceIndices() const
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FVertexInstanceID>>(MeshAttribute::Triangle::VertexInstanceIndex);
	}

	/** Accessors for array of edge IDs for triangles */
	TTriangleAttributesRef<TArrayView<FEdgeID>> GetTriangleEdgeIndices()
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FEdgeID>>(MeshAttribute::Triangle::EdgeIndex);
	}

	TTriangleAttributesConstRef<TArrayView<FEdgeID>> GetTriangleEdgeIndices() const
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FEdgeID>>(MeshAttribute::Triangle::EdgeIndex);
	}

	/** Accessors for array of vertex IDs for triangles */
	TTriangleAttributesRef<TArrayView<FVertexID>> GetTriangleVertexIndices()
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FVertexID>>(MeshAttribute::Triangle::VertexIndex);
	}

	TTriangleAttributesConstRef<TArrayView<FVertexID>> GetTriangleVertexIndices() const
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FVertexID>>(MeshAttribute::Triangle::VertexIndex);
	}

	/** Accessors for array of UV IDs for triangles */
	TTriangleAttributesRef<TArrayView<FUVID>> GetTriangleUVIndices()
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FUVID>>(MeshAttribute::Triangle::UVIndex);
	}

	TTriangleAttributesConstRef<TArrayView<FUVID>> GetTriangleUVIndices() const
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FUVID>>(MeshAttribute::Triangle::UVIndex);
	}

	/** Accessors for UV coordinates */
	TUVAttributesRef<FVector2f> GetUVCoordinates(int32 UVChannel)
	{
		return MeshDescription.UVAttributes(UVChannel).GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate);
	}

	TUVAttributesConstRef<FVector2f> GetUVCoordinates(int32 UVChannel) const
	{
		return MeshDescription.UVAttributes(UVChannel).GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate);
	}

	/** Accessors for array of polygon group IDs for triangles */
	TTriangleAttributesRef<FPolygonGroupID> GetTrianglePolygonGroupIndices()
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<FPolygonGroupID>(MeshAttribute::Triangle::PolygonGroupIndex);
	}

	TTriangleAttributesConstRef<FPolygonGroupID> GetTrianglePolygonGroupIndices() const
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<FPolygonGroupID>(MeshAttribute::Triangle::PolygonGroupIndex);
	}

	/** Accessors for array of polygon group IDs for polygons */
	TPolygonAttributesRef<FPolygonGroupID> GetPolygonPolygonGroupIndices()
	{
		return MeshDescription.PolygonAttributes().GetAttributesRef<FPolygonGroupID>(MeshAttribute::Polygon::PolygonGroupIndex);
	}

	TPolygonAttributesConstRef<FPolygonGroupID> GetPolygonPolygonGroupIndices() const
	{
		return MeshDescription.PolygonAttributes().GetAttributesRef<FPolygonGroupID>(MeshAttribute::Polygon::PolygonGroupIndex);
	}

protected:

	FMeshDescription& MeshDescription;
};


class FMeshConstAttributes
{
public:
	explicit FMeshConstAttributes(const FMeshDescription& InMeshDescription)
		: MeshDescription(InMeshDescription)
	{}

	/** Accessors for cached vertex position array */
	TVertexAttributesConstRef<FVector3f> GetVertexPositions() const { return MeshDescription.VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position); }

	/** Accessors for array of vertex IDs for vertex instances */
	TVertexInstanceAttributesConstRef<FVertexID> GetVertexInstanceVertexIndices() const
	{
		return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVertexID>(MeshAttribute::VertexInstance::VertexIndex);
	}

	/** Accessors for array of vertex IDs for edges */
	TEdgeAttributesConstRef<TArrayView<FVertexID>> GetEdgeVertexIndices() const
	{
		return MeshDescription.EdgeAttributes().GetAttributesRef<TArrayView<FVertexID>>(MeshAttribute::Edge::VertexIndex);
	}

	/** Accessors for array of vertex instance IDs for triangles */
	TTriangleAttributesConstRef<TArrayView<FVertexInstanceID>> GetTriangleVertexInstanceIndices() const
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FVertexInstanceID>>(MeshAttribute::Triangle::VertexInstanceIndex);
	}

	/** Accessors for array of edge IDs for triangles */
	TTriangleAttributesConstRef<TArrayView<FEdgeID>> GetTriangleEdgeIndices() const
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FEdgeID>>(MeshAttribute::Triangle::EdgeIndex);
	}

	/** Accessors for array of vertex IDs for triangles */
	TTriangleAttributesConstRef<TArrayView<FVertexID>> GetTriangleVertexIndices() const
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FVertexID>>(MeshAttribute::Triangle::VertexIndex);
	}

	/** Accessors for array of UV IDs for triangles */
	TTriangleAttributesConstRef<TArrayView<FUVID>> GetTriangleUVIndices() const
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<TArrayView<FUVID>>(MeshAttribute::Triangle::UVIndex);
	}

	/** Accessors for UV coordinates */
	TUVAttributesConstRef<FVector2f> GetUVCoordinates(int32 UVChannel) const
	{
		return MeshDescription.UVAttributes(UVChannel).GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate);
	}

	/** Accessors for array of polygon group IDs for triangles */
	TTriangleAttributesConstRef<FPolygonGroupID> GetTrianglePolygonGroupIndices() const
	{
		return MeshDescription.TriangleAttributes().GetAttributesRef<FPolygonGroupID>(MeshAttribute::Triangle::PolygonGroupIndex);
	}

	/** Accessors for array of polygon group IDs for polygons */
	TPolygonAttributesConstRef<FPolygonGroupID> GetPolygonPolygonGroupIndices() const
	{
		return MeshDescription.PolygonAttributes().GetAttributesRef<FPolygonGroupID>(MeshAttribute::Polygon::PolygonGroupIndex);
	}

protected:

	const FMeshDescription& MeshDescription;
};
