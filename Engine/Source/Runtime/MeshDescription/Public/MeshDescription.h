// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Accumulate.h"
#include "Algo/Copy.h"
#include "Algo/Find.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StaticArray.h"
#include "Containers/UnrealString.h"
#include "CoreFwd.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "Math/Box.h"
#include "Math/MathFwd.h"
#include "Math/Plane.h"
#include "Math/Vector.h"
#include "MeshAttributeArray.h"
#include "MeshElementArray.h"
#include "MeshElementContainer.h"
#include "MeshElementIndexer.h"
#include "MeshTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/EditorBulkData.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ReleaseObjectVersion.h"

#include "MeshDescription.generated.h"

class FArchive;
class UObject;
struct FElementIDRemappings;

enum
{
	//Remove the _MD when FRawMesh will be remove
	MAX_MESH_TEXTURE_COORDS_MD = 8,
};


/** Define container types */
using FVertexArray = TMeshElementContainer<FVertexID>;
using FVertexInstanceArray = TMeshElementContainer<FVertexInstanceID>;
using FEdgeArray = TMeshElementContainer<FEdgeID>;
using FUVArray = TMeshElementContainer<FUVID>;
using FTriangleArray = TMeshElementContainer<FTriangleID>;
using FPolygonArray = TMeshElementContainer<FPolygonID>;
using FPolygonGroupArray = TMeshElementContainer<FPolygonGroupID>;

/** Define aliases for element attributes */
template <typename AttributeType> using TVertexAttributesRef = TMeshAttributesRef<FVertexID, AttributeType>;
template <typename AttributeType> using TVertexInstanceAttributesRef = TMeshAttributesRef<FVertexInstanceID, AttributeType>;
template <typename AttributeType> using TEdgeAttributesRef = TMeshAttributesRef<FEdgeID, AttributeType>;
template <typename AttributeType> using TUVAttributesRef = TMeshAttributesRef<FUVID, AttributeType>;
template <typename AttributeType> using TTriangleAttributesRef = TMeshAttributesRef<FTriangleID, AttributeType>;
template <typename AttributeType> using TPolygonAttributesRef = TMeshAttributesRef<FPolygonID, AttributeType>;
template <typename AttributeType> using TPolygonGroupAttributesRef = TMeshAttributesRef<FPolygonGroupID, AttributeType>;

template <typename AttributeType> using TVertexAttributesConstRef = TMeshAttributesConstRef<FVertexID, AttributeType>;
template <typename AttributeType> using TVertexInstanceAttributesConstRef = TMeshAttributesConstRef<FVertexInstanceID, AttributeType>;
template <typename AttributeType> using TEdgeAttributesConstRef = TMeshAttributesConstRef<FEdgeID, AttributeType>;
template <typename AttributeType> using TUVAttributesConstRef = TMeshAttributesConstRef<FUVID, AttributeType>;
template <typename AttributeType> using TTriangleAttributesConstRef = TMeshAttributesConstRef<FTriangleID, AttributeType>;
template <typename AttributeType> using TPolygonAttributesConstRef = TMeshAttributesConstRef<FPolygonID, AttributeType>;
template <typename AttributeType> using TPolygonGroupAttributesConstRef = TMeshAttributesConstRef<FPolygonGroupID, AttributeType>;

UENUM()
enum class EComputeNTBsOptions : uint32
{
	None = 0x00000000,	// No flags
	Normals = 0x00000001, //Compute the normals
	Tangents = 0x00000002, //Compute the tangents
	WeightedNTBs = 0x00000004, //Use weight angle when computing NTBs to proportionally distribute the vertex instance contribution to the normal/tangent/binormal in a smooth group.    i.e. Weight solve the cylinder problem
};
ENUM_CLASS_FLAGS(EComputeNTBsOptions);


struct FMeshDescription
{
public:

	// Mesh description should be a moveable type.
	// Hence explicitly declare all the below as defaulted, to ensure they will be generated by the compiler.
	MESHDESCRIPTION_API FMeshDescription();
	~FMeshDescription() = default;
	MESHDESCRIPTION_API FMeshDescription(const FMeshDescription&);
	MESHDESCRIPTION_API FMeshDescription& operator=(const FMeshDescription&);
	FMeshDescription(FMeshDescription&&) = default;
	FMeshDescription& operator=(FMeshDescription&&) = default;

	friend FArchive& operator<<(FArchive& Ar, FMeshDescription& MeshDescription)
	{
		MeshDescription.Serialize(Ar);
		return Ar;
	}

	MESHDESCRIPTION_API static TConstArrayView<TPair<FGuid, int32>> GetSerializationFormatVersions();

	// Serialize the mesh description
	MESHDESCRIPTION_API void Serialize(FArchive& Ar);

	// Legacy serialization for old assets
	MESHDESCRIPTION_API void SerializeLegacy(FArchive& Ar);

	// Empty the meshdescription
	MESHDESCRIPTION_API void Empty();

	// Operations on all the indexers
	MESHDESCRIPTION_API void ResetIndexers();
	MESHDESCRIPTION_API void BuildIndexers();
	MESHDESCRIPTION_API void RebuildIndexers();

	// Return whether the mesh description is empty
	MESHDESCRIPTION_API bool IsEmpty() const;

	FVertexArray& Vertices() { return static_cast<TMeshElementContainer<FVertexID>&>(VertexElements->Get()); }
	const FVertexArray& Vertices() const { return static_cast<const TMeshElementContainer<FVertexID>&>(VertexElements->Get()); }

	FVertexInstanceArray& VertexInstances() { return static_cast<TMeshElementContainer<FVertexInstanceID>&>(VertexInstanceElements->Get()); }
	const FVertexInstanceArray& VertexInstances() const { return static_cast<const TMeshElementContainer<FVertexInstanceID>&>(VertexInstanceElements->Get()); }

	FEdgeArray& Edges() { return static_cast<TMeshElementContainer<FEdgeID>&>(EdgeElements->Get()); }
	const FEdgeArray& Edges() const { return static_cast<const TMeshElementContainer<FEdgeID>&>(EdgeElements->Get()); }

	FUVArray& UVs(int32 Index) { return static_cast<TMeshElementContainer<FUVID>&>(UVElements->Get(Index)); }
	const FUVArray& UVs(int32 Index) const { return static_cast<const TMeshElementContainer<FUVID>&>(UVElements->Get(Index)); }

	FTriangleArray& Triangles() { return static_cast<TMeshElementContainer<FTriangleID>&>(TriangleElements->Get()); }
	const FTriangleArray& Triangles() const { return static_cast<const TMeshElementContainer<FTriangleID>&>(TriangleElements->Get()); }

	FPolygonArray& Polygons() { return static_cast<TMeshElementContainer<FPolygonID>&>(PolygonElements->Get()); }
	const FPolygonArray& Polygons() const { return static_cast<const TMeshElementContainer<FPolygonID>&>(PolygonElements->Get()); }

	FPolygonGroupArray& PolygonGroups() { return static_cast<TMeshElementContainer<FPolygonGroupID>&>(PolygonGroupElements->Get()); }
	const FPolygonGroupArray& PolygonGroups() const { return static_cast<const TMeshElementContainer<FPolygonGroupID>&>(PolygonGroupElements->Get()); }

	TAttributesSet<FVertexID>& VertexAttributes() { return Vertices().GetAttributes(); }
	const TAttributesSet<FVertexID>& VertexAttributes() const { return Vertices().GetAttributes(); }

	TAttributesSet<FVertexInstanceID>& VertexInstanceAttributes() { return VertexInstances().GetAttributes(); }
	const TAttributesSet<FVertexInstanceID>& VertexInstanceAttributes() const { return VertexInstances().GetAttributes(); }

	TAttributesSet<FEdgeID>& EdgeAttributes() { return Edges().GetAttributes(); }
	const TAttributesSet<FEdgeID>& EdgeAttributes() const { return Edges().GetAttributes(); }

	TAttributesSet<FUVID>& UVAttributes(int32 Index) { return UVs(Index).GetAttributes(); }
	const TAttributesSet<FUVID>& UVAttributes(int32 Index) const { return UVs(Index).GetAttributes(); }

	TAttributesSet<FTriangleID>& TriangleAttributes() { return Triangles().GetAttributes(); }
	const TAttributesSet<FTriangleID>& TriangleAttributes() const { return Triangles().GetAttributes(); }

	TAttributesSet<FPolygonID>& PolygonAttributes() { return Polygons().GetAttributes(); }
	const TAttributesSet<FPolygonID>& PolygonAttributes() const { return Polygons().GetAttributes(); }

	TAttributesSet<FPolygonGroupID>& PolygonGroupAttributes() { return PolygonGroups().GetAttributes(); }
	const TAttributesSet<FPolygonGroupID>& PolygonGroupAttributes() const { return PolygonGroups().GetAttributes(); }

	TMap<FName, FMeshElementTypeWrapper>& GetElements() { return Elements; }
	const TMap<FName, FMeshElementTypeWrapper>& GetElements() const { return Elements; }

	void SuspendVertexIndexing() { VertexToVertexInstances.Suspend(); VertexToEdges.Suspend(); }
	void SuspendVertexInstanceIndexing() { VertexInstanceToTriangles.Suspend(); }
	void SuspendEdgeIndexing() { EdgeToTriangles.Suspend(); }
	void SuspendPolygonIndexing() { PolygonToTriangles.Suspend(); }
	void SuspendPolygonGroupIndexing() { PolygonGroupToPolygons.Suspend(); PolygonGroupToTriangles.Suspend(); }
	void SuspendUVIndexing() { UVToTriangles.Suspend(); }

	void ResumeVertexIndexing() { VertexToVertexInstances.Resume(); VertexToEdges.Resume(); }
	void ResumeVertexInstanceIndexing() { VertexInstanceToTriangles.Resume(); }
	void ResumeEdgeIndexing() { EdgeToTriangles.Resume(); }
	void ResumePolygonIndexing() { PolygonToTriangles.Resume(); }
	void ResumePolygonGroupIndexing() { PolygonGroupToPolygons.Resume(); PolygonGroupToTriangles.Resume(); }
	void ResumeUVIndexing() { UVToTriangles.Resume(); }

	void BuildVertexIndexers() { VertexToVertexInstances.Build(); VertexToEdges.Build(); }
	void BuildVertexInstanceIndexers() { VertexInstanceToTriangles.Build(); }
	void BuildEdgeIndexers() { EdgeToTriangles.Build(); }
	void BuildPolygonIndexers() { PolygonToTriangles.Build(); }
	void BuildPolygonGroupIndexers() { PolygonGroupToPolygons.Build(), PolygonGroupToTriangles.Build(); }

//////////////////////////////////////////////////////////////////////////
// Create / remove mesh elements

	/** Reserves space for this number of new vertices */
	void ReserveNewVertices(const int32 NumVertices)
	{
		VertexElements->Get().Reserve(VertexElements->Get().Num() + NumVertices);
	}

	/** Adds a new vertex to the mesh and returns its ID */
	FVertexID CreateVertex()
	{
		const FVertexID VertexID = VertexElements->Get().Add();
		return VertexID;
	}

	/** Adds a new vertex to the mesh with the given ID */
	void CreateVertexWithID(const FVertexID VertexID)
	{
		VertexElements->Get().Insert(VertexID);
	}

	/** Deletes a vertex from the mesh */
	void DeleteVertex(const FVertexID VertexID)
	{
		check(VertexToVertexInstances.Find(VertexID).Num() == 0);
		check(VertexToEdges.Find(VertexID).Num() == 0);
		VertexElements->Get().Remove(VertexID);
		VertexToVertexInstances.RemoveKey(VertexID);
		VertexToEdges.RemoveKey(VertexID);
	}

	/** Returns whether the passed vertex ID is valid */
	bool IsVertexValid(const FVertexID VertexID) const
	{
		return VertexElements->Get().IsValid(VertexID);
	}

	/** Reserves space for this number of new vertex instances */
	void ReserveNewVertexInstances(const int32 NumVertexInstances)
	{
		VertexInstanceElements->Get().Reserve(VertexInstanceElements->Get().Num() + NumVertexInstances);
	}

	/** Adds a new vertex instance to the mesh and returns its ID */
	FVertexInstanceID CreateVertexInstance(const FVertexID VertexID)
	{
		const FVertexInstanceID VertexInstanceID = VertexInstanceElements->Get().Add();
		CreateVertexInstance_Internal(VertexInstanceID, VertexID);
		return VertexInstanceID;
	}

	/** Adds a new vertex instance to the mesh with the given ID */
	void CreateVertexInstanceWithID(const FVertexInstanceID VertexInstanceID, const FVertexID VertexID)
	{
		VertexInstanceElements->Get().Insert(VertexInstanceID);
		CreateVertexInstance_Internal(VertexInstanceID, VertexID);
	}

	/** Deletes a vertex instance from a mesh */
	MESHDESCRIPTION_API void DeleteVertexInstance(const FVertexInstanceID VertexInstanceID, TArray<FVertexID>* InOutOrphanedVerticesPtr = nullptr);
	
	/** Returns whether the passed vertex instance ID is valid */
	bool IsVertexInstanceValid(const FVertexInstanceID VertexInstanceID) const
	{
		return VertexInstanceElements->Get().IsValid(VertexInstanceID);
	}

	/** Reserves space for this number of new UVs */
	void ReserveNewUVs(const int32 NumUVs, const int32 UVChannel = 0)
	{
		UVElements->Get(UVChannel).Reserve(UVElements->Get(UVChannel).Num() + NumUVs);
	}

	/** Adds a new UV to the mesh and returns its ID */
	FUVID CreateUV(const int32 UVChannel = 0)
	{
		const FUVID UVID = UVElements->Get(UVChannel).Add();
		return UVID;
	}

	/** Adds a new UV to the mesh with the given ID */
	void CreateUVWithID(const FUVID UVID, const int32 UVChannel = 0)
	{
		UVElements->Get(UVChannel).Insert(UVID);
	}

	/** Deletes a UV from the mesh */
	void DeleteUV(const FUVID UVID, const int32 UVChannel = 0)
	{
		check(UVToTriangles.Find(UVID).Num() == 0);
		UVElements->Get(UVChannel).Remove(UVID);
		UVToTriangles.RemoveKey(UVID);
	}

	/** Returns whether the passed UV ID is valid */
	bool IsUVValid(const FUVID UVID, const int32 UVChannel = 0) const
	{
		return UVElements->Get(UVChannel).IsValid(UVID);
	}

	/** Reserves space for this number of new edges */
	void ReserveNewEdges(const int32 NumEdges)
	{
		EdgeElements->Get().Reserve(EdgeElements->Get().Num() + NumEdges);
	}

	/** Adds a new edge to the mesh and returns its ID */
	FEdgeID CreateEdge(const FVertexID VertexID0, const FVertexID VertexID1)
	{
		const FEdgeID EdgeID = EdgeElements->Get().Add();
		CreateEdge_Internal(EdgeID, VertexID0, VertexID1);
		return EdgeID;
	}

	/** Adds a new edge to the mesh with the given ID */
	void CreateEdgeWithID(const FEdgeID EdgeID, const FVertexID VertexID0, const FVertexID VertexID1)
	{
		EdgeElements->Get().Insert(EdgeID);
		CreateEdge_Internal(EdgeID, VertexID0, VertexID1);
	}

	/** Deletes an edge from the mesh */
	MESHDESCRIPTION_API void DeleteEdge(const FEdgeID EdgeID, TArray<FVertexID>* InOutOrphanedVerticesPtr = nullptr);

	/** Returns whether the passed edge ID is valid */
	bool IsEdgeValid(const FEdgeID EdgeID) const
	{
		return EdgeElements->Get().IsValid(EdgeID);
	}

	/** Reserves space for this number of new triangles */
	void ReserveNewTriangles(const int32 NumTriangles)
	{
		TriangleElements->Get().Reserve(TriangleElements->Get().Num() + NumTriangles);
	}

	/** Adds a new triangle to the mesh and returns its ID. This will also make an encapsulating polygon, and any missing edges. */
	FTriangleID CreateTriangle(const FPolygonGroupID PolygonGroupID, TArrayView<const FVertexInstanceID> VertexInstanceIDs, TArray<FEdgeID>* OutEdgeIDs = nullptr)
	{
		const FTriangleID TriangleID = TriangleElements->Get().Add();
		CreateTriangle_Internal(TriangleID, PolygonGroupID, VertexInstanceIDs, OutEdgeIDs);
		return TriangleID;
	}

	/** Adds a new triangle to the mesh with the given ID. This will also make an encapsulating polygon, and any missing edges. */
	void CreateTriangleWithID(const FTriangleID TriangleID, const FPolygonGroupID PolygonGroupID, TArrayView<const FVertexInstanceID> VertexInstanceIDs, TArray<FEdgeID>* OutEdgeIDs = nullptr)
	{
		TriangleElements->Get().Insert(TriangleID);
		CreateTriangle_Internal(TriangleID, PolygonGroupID, VertexInstanceIDs, OutEdgeIDs);
	}

	/** Deletes a triangle from the mesh */
	MESHDESCRIPTION_API void DeleteTriangle(const FTriangleID TriangleID, TArray<FEdgeID>* InOutOrphanedEdgesPtr = nullptr, TArray<FVertexInstanceID>* InOutOrphanedVertexInstancesPtr = nullptr, TArray<FPolygonGroupID>* InOutOrphanedPolygonGroupsPtr = nullptr);

	/** Deletes triangles from the mesh and remove all orphaned polygon groups, vertex instances, edges and vertices.
		Will not compact the internal arrays, you must call Compact() manually.
	 */
	MESHDESCRIPTION_API void DeleteTriangles(const TArray<FTriangleID>& Triangles);

	/** Returns whether the passed triangle ID is valid */
	bool IsTriangleValid(const FTriangleID TriangleID) const
	{
		return TriangleElements->Get().IsValid(TriangleID);
	}

	/** Reserves space for this number of new polygons */
	void ReserveNewPolygons(const int32 NumPolygons)
	{
		PolygonElements->Get().Reserve(PolygonElements->Get().Num() + NumPolygons);
	}

	/** Adds a new polygon to the mesh and returns its ID. This will also make any missing edges, and all constituent triangles. */
	FPolygonID CreatePolygon(const FPolygonGroupID PolygonGroupID, TArrayView<const FVertexInstanceID> VertexInstanceIDs, TArray<FEdgeID>* OutEdgeIDs = nullptr)
	{
		const FPolygonID PolygonID = PolygonElements->Get().Add();
		CreatePolygon_Internal(PolygonID, PolygonGroupID, VertexInstanceIDs, OutEdgeIDs);
		return PolygonID;
	}

	/** Adds a new polygon to the mesh with the given ID. This will also make any missing edges, and all constituent triangles. */
	void CreatePolygonWithID(const FPolygonID PolygonID, const FPolygonGroupID PolygonGroupID, TArrayView<const FVertexInstanceID> VertexInstanceIDs, TArray<FEdgeID>* OutEdgeIDs = nullptr)
	{
		PolygonElements->Get().Insert(PolygonID);
		CreatePolygon_Internal(PolygonID, PolygonGroupID, VertexInstanceIDs, OutEdgeIDs);
	}

	/** Deletes a polygon from the mesh */
	MESHDESCRIPTION_API void DeletePolygon(const FPolygonID PolygonID, TArray<FEdgeID>* InOutOrphanedEdgesPtr = nullptr, TArray<FVertexInstanceID>* InOutOrphanedVertexInstancesPtr = nullptr, TArray<FPolygonGroupID>* InOutOrphanedPolygonGroupsPtr = nullptr);

	/** Deletes polygons from the mesh and remove all orphaned polygon groups, vertex instances, edges and vertices.
		Will not compact the internal arrays, you must call Compact() manually.
	 */
	MESHDESCRIPTION_API void DeletePolygons(const TArray<FPolygonID>& Polygons);

	/** Returns whether the passed polygon ID is valid */
	bool IsPolygonValid(const FPolygonID PolygonID) const
	{
		return PolygonElements->Get().IsValid(PolygonID);
	}

	/** Reserves space for this number of new polygon groups */
	void ReserveNewPolygonGroups(const int32 NumPolygonGroups)
	{
		PolygonGroupElements->Get().Reserve(PolygonGroupElements->Get().Num() + NumPolygonGroups);
	}

	/** Adds a new polygon group to the mesh and returns its ID */
	FPolygonGroupID CreatePolygonGroup()
	{
		const FPolygonGroupID PolygonGroupID = PolygonGroupElements->Get().Add();
		return PolygonGroupID;
	}

	/** Adds a new polygon group to the mesh with the given ID */
	void CreatePolygonGroupWithID(const FPolygonGroupID PolygonGroupID)
	{
		PolygonGroupElements->Get().Insert(PolygonGroupID);
	}

	/** Deletes a polygon group from the mesh */
	void DeletePolygonGroup(const FPolygonGroupID PolygonGroupID)
	{
		check(PolygonGroupToPolygons.Find(PolygonGroupID).Num() == 0);
		PolygonGroupElements->Get().Remove(PolygonGroupID);
		PolygonGroupToPolygons.RemoveKey(PolygonGroupID);
		PolygonGroupToTriangles.RemoveKey(PolygonGroupID);
	}

	/** Returns whether the passed polygon group ID is valid */
	bool IsPolygonGroupValid(const FPolygonGroupID PolygonGroupID) const
	{
		return PolygonGroupElements->Get().IsValid(PolygonGroupID);
	}


//////////////////////////////////////////////////////////////////////////
// MeshDescription general functions

public:

	//////////////////////////////////////////////////////////////////////
	// Vertex operations

	/** Returns whether a given vertex is orphaned, i.e. it doesn't form part of any polygon */
	MESHDESCRIPTION_API bool IsVertexOrphaned(const FVertexID VertexID) const;

	/** Returns the edge ID defined by the two given vertex IDs, if there is one; otherwise INDEX_NONE */
	MESHDESCRIPTION_API FEdgeID GetVertexPairEdge(const FVertexID VertexID0, const FVertexID VertexID1) const;

	/** Returns reference to an array of Edge IDs connected to this vertex */
	TArrayView<const FEdgeID> GetVertexConnectedEdgeIDs(const FVertexID VertexID) const
	{
		return VertexToEdges.Find<FEdgeID>(VertexID);
	}

	UE_DEPRECATED(4.26, "Please use GetVertexConnectedEdgeIDs instead.")
	TArray<FEdgeID> GetVertexConnectedEdges(const FVertexID VertexID) const
	{
		return TArray<FEdgeID>(GetVertexConnectedEdgeIDs(VertexID));
	}

	/** Returns number of edges connected to this vertex */
	int32 GetNumVertexConnectedEdges(const FVertexID VertexID) const
	{
		return VertexToEdges.Find(VertexID).Num();
	}

	/** Returns reference to an array of VertexInstance IDs instanced from this vertex */
	TArrayView<const FVertexInstanceID> GetVertexVertexInstanceIDs(const FVertexID VertexID) const
	{
		return VertexToVertexInstances.Find<FVertexInstanceID>(VertexID);
	}

	UE_DEPRECATED(4.26, "Please use GetVertexVertexInstanceIDs instead.")
	TArray<FVertexInstanceID> GetVertexVertexInstances(const FVertexID VertexID) const
	{
		return TArray<FVertexInstanceID>(GetVertexVertexInstanceIDs(VertexID));
	}

	/** Returns number of vertex instances created from this vertex */
	int32 GetNumVertexVertexInstances(const FVertexID VertexID) const
	{
		return VertexToVertexInstances.Find(VertexID).Num();
	}

	/** Populates the passed array of TriangleIDs with the triangles connected to this vertex */
	template <typename Alloc>
	void GetVertexConnectedTriangles(const FVertexID VertexID, TArray<FTriangleID, Alloc>& OutConnectedTriangleIDs) const
	{
		OutConnectedTriangleIDs.Reset(GetNumVertexConnectedTriangles(VertexID));
		for (const FVertexInstanceID& VertexInstanceID : VertexToVertexInstances.Find<FVertexInstanceID>(VertexID))
		{
			TArrayView<const FTriangleID> ConnectedTris = VertexInstanceToTriangles.Find<FTriangleID>(VertexInstanceID);
			OutConnectedTriangleIDs.Append(ConnectedTris.GetData(), ConnectedTris.Num());
		}
	}

	/** Returns the triangles connected to this vertex as an array with the specified allocator template type. */
	template <typename Alloc>
	TArray<FTriangleID, Alloc> GetVertexConnectedTriangles(const FVertexID VertexID) const
	{
		TArray<FTriangleID, Alloc> Result;
		this->GetVertexConnectedTriangles(VertexID, Result);
		return Result;
	}

	/** Returns the triangles connected to this vertex */
	TArray<FTriangleID> GetVertexConnectedTriangles(const FVertexID VertexID) const
	{
		TArray<FTriangleID> Result;
		this->GetVertexConnectedTriangles(VertexID, Result);
		return Result;
	}

	/** Returns number of triangles connected to this vertex */
	int32 GetNumVertexConnectedTriangles(const FVertexID VertexID) const
	{
		TArrayView<const FVertexInstanceID> VertexInstances = VertexToVertexInstances.Find<FVertexInstanceID>(VertexID);
		return Algo::TransformAccumulate(VertexInstances,
										 [this](const FVertexInstanceID ID) { return VertexInstanceToTriangles.Find(ID).Num(); },
										 0);
	}

	/** Populates the passed array of PolygonIDs with the polygons connected to this vertex */
	template <typename Alloc>
	void GetVertexConnectedPolygons(const FVertexID VertexID, TArray<FPolygonID, Alloc>& OutConnectedPolygonIDs) const
	{
		OutConnectedPolygonIDs.Reset();
		for (const FVertexInstanceID& VertexInstanceID : VertexToVertexInstances.Find<FVertexInstanceID>(VertexID))
		{
			for (const FTriangleID& TriangleID : VertexInstanceToTriangles.Find<FTriangleID>(VertexInstanceID))
			{
				OutConnectedPolygonIDs.AddUnique(TrianglePolygons[TriangleID]);
			}
		}
	}

	/** Returns the polygons connected to this vertex as an array with the specified allocator template type. */
	template <typename Alloc>
	TArray<FPolygonID, Alloc> GetVertexConnectedPolygons(const FVertexID VertexID) const
	{
		TArray<FPolygonID, Alloc> Result;
		this->GetVertexConnectedPolygons(VertexID, Result);
		return Result;
	}

	/** Returns the polygons connected to this vertex */
	TArray<FPolygonID> GetVertexConnectedPolygons(const FVertexID VertexID) const
	{
		TArray<FPolygonID> Result;
		this->GetVertexConnectedPolygons(VertexID, Result);
		return Result;
	}

	/** Returns the number of polygons connected to this vertex */
	int32 GetNumVertexConnectedPolygons(const FVertexID VertexID) const
	{
		return GetVertexConnectedPolygons<TInlineAllocator<8>>(VertexID).Num();
	}

	/** Populates the passed array of VertexIDs with the vertices adjacent to this vertex */
	template <typename Alloc>
	void GetVertexAdjacentVertices(const FVertexID VertexID, TArray<FVertexID, Alloc>& OutAdjacentVertexIDs) const
	{
		TArrayView<const FEdgeID> ConnectedEdgeIDs = VertexToEdges.Find<FEdgeID>(VertexID);
		OutAdjacentVertexIDs.SetNumUninitialized(ConnectedEdgeIDs.Num());

		int32 Index = 0;
		for (const FEdgeID& EdgeID : ConnectedEdgeIDs)
		{
			TArrayView<const FVertexID> EdgeVertexIDs = EdgeVertices[EdgeID];
			OutAdjacentVertexIDs[Index] = (EdgeVertexIDs[0] == VertexID) ? EdgeVertexIDs[1] : EdgeVertexIDs[0];
			Index++;
		}
	}

	/** Returns the vertices adjacent to this vertex as an array with the specified allocator template type. */
	template <typename Alloc>
	TArray<FVertexID, Alloc> GetVertexAdjacentVertices(const FVertexID VertexID) const
	{
		TArray<FVertexID, Alloc> Result;
		this->GetVertexAdjacentVertices(VertexID, Result);
		return Result;
	}

	/** Returns the vertices adjacent to this vertex */
	TArray<FVertexID> GetVertexAdjacentVertices(const FVertexID VertexID) const
	{
		TArray<FVertexID> Result;
		this->GetVertexAdjacentVertices(VertexID, Result);
		return Result;
	}

	/** Returns the position of this vertex */
	FVector3f GetVertexPosition(const FVertexID VertexID) const
	{
		return VertexPositions[VertexID];
	}

	TVertexAttributesRef<FVector3f> GetVertexPositions()
	{
		return VertexPositions;
	}

	TVertexAttributesRef<const FVector3f> GetVertexPositions() const
	{
		return VertexPositions;
	}

	//////////////////////////////////////////////////////////////////////
	// Vertex instance operations

	/** Returns the vertex ID associated with the given vertex instance */
	FVertexID GetVertexInstanceVertex(const FVertexInstanceID VertexInstanceID) const
	{
		return VertexInstanceVertices[VertexInstanceID];
	}

	/** Returns the edge ID defined by the two given vertex instance IDs, if there is one; otherwise INDEX_NONE */
	MESHDESCRIPTION_API FEdgeID GetVertexInstancePairEdge(const FVertexInstanceID VertexInstanceID0, const FVertexInstanceID VertexInstanceID1) const;

	/** Returns reference to an array of Triangle IDs connected to this vertex instance */
	TArrayView<const FTriangleID> GetVertexInstanceConnectedTriangleIDs(const FVertexInstanceID VertexInstanceID) const
	{
		return VertexInstanceToTriangles.Find<FTriangleID>(VertexInstanceID);
	}

	UE_DEPRECATED(4.26, "Please use GetVertexInstanceTriangleIDs() instead.")
	TArray<FTriangleID> GetVertexInstanceConnectedTriangles(const FVertexInstanceID VertexInstanceID) const
	{
		return TArray<FTriangleID>(GetVertexInstanceConnectedTriangleIDs(VertexInstanceID));
	}

	/** Returns the number of triangles connected to this vertex instance */
	int32 GetNumVertexInstanceConnectedTriangles(const FVertexInstanceID VertexInstanceID) const
	{
		return VertexInstanceToTriangles.Find(VertexInstanceID).Num();
	}

	/** Populates the passed array with the polygons connected to this vertex instance */
	template <typename Alloc>
	void GetVertexInstanceConnectedPolygons(const FVertexInstanceID VertexInstanceID, TArray<FPolygonID, Alloc>& OutPolygonIDs) const
	{
		OutPolygonIDs.Reset(VertexInstanceToTriangles.Find(VertexInstanceID).Num());
		for (const FTriangleID& TriangleID : VertexInstanceToTriangles.Find<FTriangleID>(VertexInstanceID))
		{
			OutPolygonIDs.AddUnique(TrianglePolygons[TriangleID]);
		}
	}

	/** Returns the polygons connected to this vertex instance as an array with the specified allocator template type. */
	template <typename Alloc>
	TArray<FPolygonID, Alloc> GetVertexInstanceConnectedPolygons(const FVertexInstanceID VertexInstanceID) const
	{
		TArray<FPolygonID, Alloc> Result;
		this->GetVertexInstanceConnectedPolygons(VertexInstanceID, Result);
		return Result;
	}

	/** Returns the polygons connected to this vertex instance */
	TArray<FPolygonID> GetVertexInstanceConnectedPolygons(const FVertexInstanceID VertexInstanceID) const
	{
		TArray<FPolygonID> Result;
		this->GetVertexInstanceConnectedPolygons(VertexInstanceID, Result);
		return Result;
	}

	/** Returns the number of polygons connected to this vertex instance. */
	int32 GetNumVertexInstanceConnectedPolygons(const FVertexInstanceID VertexInstanceID) const
	{
		return GetVertexInstanceConnectedPolygons<TInlineAllocator<8>>(VertexInstanceID).Num();
	}


	//////////////////////////////////////////////////////////////////////
	// Edge operations

	/** Determine whether a given edge is an internal edge between triangles of a polygon */
	bool IsEdgeInternal(const FEdgeID EdgeID) const
	{
		TArrayView<const FTriangleID> ConnectedTriangles = EdgeToTriangles.Find<FTriangleID>(EdgeID);
		return ConnectedTriangles.Num() == 2 &&
			   TrianglePolygons[ConnectedTriangles[0]] == TrianglePolygons[ConnectedTriangles[1]];
	}

	/** Determine whether a given edge is an internal edge between triangles of a specific polygon */
	bool IsEdgeInternalToPolygon(const FEdgeID EdgeID, const FPolygonID PolygonID) const
	{
		TArrayView<const FTriangleID> ConnectedTriangles = EdgeToTriangles.Find<FTriangleID>(EdgeID);
		return ConnectedTriangles.Num() == 2 &&
			   TrianglePolygons[ConnectedTriangles[0]] == PolygonID &&
			   TrianglePolygons[ConnectedTriangles[1]] == PolygonID;
	}

	/** Returns reference to an array of triangle IDs connected to this edge */
	TArrayView<const FTriangleID> GetEdgeConnectedTriangleIDs(const FEdgeID EdgeID) const
	{
		return EdgeToTriangles.Find<FTriangleID>(EdgeID);
	}

	UE_DEPRECATED(4.26, "Please use GetEdgeConnectedTriangleIDs() instead.")
	TArray<FTriangleID> GetEdgeConnectedTriangles(const FEdgeID EdgeID) const
	{
		return TArray<FTriangleID>(GetEdgeConnectedTriangleIDs(EdgeID));
	}

	int32 GetNumEdgeConnectedTriangles(const FEdgeID EdgeID) const
	{
		return EdgeToTriangles.Find(EdgeID).Num();
	}

	/** Populates the passed array with polygon IDs connected to this edge */
	template <typename Alloc>
	void GetEdgeConnectedPolygons(const FEdgeID EdgeID, TArray<FPolygonID, Alloc>& OutPolygonIDs) const
	{
		OutPolygonIDs.Reset(EdgeToTriangles.Find(EdgeID).Num());
		for (const FTriangleID& TriangleID : EdgeToTriangles.Find<FTriangleID>(EdgeID))
		{
			OutPolygonIDs.AddUnique(TrianglePolygons[TriangleID]);
		}
	}

	/** Returns the polygons connected to this edge as an array with the specified allocator template type. */
	template <typename Alloc>
	TArray<FPolygonID, Alloc> GetEdgeConnectedPolygons(const FEdgeID EdgeID) const
	{
		TArray<FPolygonID, Alloc> Result;
		this->GetEdgeConnectedPolygons(EdgeID, Result);
		return Result;
	}

	/** Returns the polygons connected to this edge */
	TArray<FPolygonID> GetEdgeConnectedPolygons(const FEdgeID EdgeID) const
	{
		TArray<FPolygonID> Result;
		this->GetEdgeConnectedPolygons(EdgeID, Result);
		return Result;
	}

	/** Returns the number of polygons connected to this edge */
	int32 GetNumEdgeConnectedPolygons(const FEdgeID EdgeID) const
	{
		return GetEdgeConnectedPolygons<TInlineAllocator<8>>(EdgeID).Num();
	}

	/** Returns the vertex ID corresponding to one of the edge endpoints */
	FVertexID GetEdgeVertex(const FEdgeID EdgeID, int32 VertexNumber) const
	{
		check(VertexNumber == 0 || VertexNumber == 1);
		TArrayView<const FVertexID> EdgeVertexIDs = EdgeVertices[EdgeID];
		return EdgeVertexIDs[VertexNumber];
	}

	/** Returns a pair of vertex IDs defining the edge */
	TArrayView<const FVertexID> GetEdgeVertices(const FEdgeID EdgeID) const
	{
		return EdgeVertices[EdgeID];
	}


	//////////////////////////////////////////////////////////////////////
	// Triangle operations

	/** Get the polygon which contains this triangle */
	FPolygonID GetTrianglePolygon(const FTriangleID TriangleID) const
	{
		return TrianglePolygons[TriangleID];
	}

	/** Get the polygon group which contains this triangle */
	FPolygonGroupID GetTrianglePolygonGroup(const FTriangleID TriangleID) const
	{
		return TrianglePolygonGroups[TriangleID];
	}

	/** Determines if this triangle is part of an n-gon */
	bool IsTrianglePartOfNgon(const FTriangleID TriangleID) const
	{
		return PolygonToTriangles.Find(TrianglePolygons[TriangleID]).Num() > 1;
	}

	/** Get the vertex instances which define this triangle */
	TArrayView<const FVertexInstanceID> GetTriangleVertexInstances(const FTriangleID TriangleID) const
	{
		return TriangleVertexInstances[TriangleID];
	}

	/** Get the specified vertex instance by index */
	FVertexInstanceID GetTriangleVertexInstance(const FTriangleID TriangleID, const int32 Index) const
	{
		check(Index >= 0 && Index < 3);
		TArrayView<const FVertexInstanceID> TriVertexInstanceIDs = TriangleVertexInstances[TriangleID];
		return TriVertexInstanceIDs[Index];
	}

	/** Populates the passed array with the vertices which define this triangle */
	UE_DEPRECATED(4.26, "Use the other form of GetTriangleVertices")
	void GetTriangleVertices(const FTriangleID TriangleID, TArrayView<FVertexID> OutVertexIDs) const
	{
		check(OutVertexIDs.Num() >= 3);
		TArrayView<const FVertexID> TriVerts = TriangleVertices[TriangleID];
		for (int32 Index = 0; Index < 3; ++Index)
		{
			OutVertexIDs[Index] = TriVerts[Index];
		}
	}

	/** Return the vertices which define this triangle */
	TArrayView<const FVertexID> GetTriangleVertices(const FTriangleID TriangleID) const
	{
		return TriangleVertices[TriangleID];
	}

	/** Populates the passed array with the edges which define this triangle */
	UE_DEPRECATED(4.26, "Use the other form of GetTriangleEdges")
	void GetTriangleEdges(const FTriangleID TriangleID, TArrayView<FEdgeID> OutEdgeIDs) const
	{
		check(OutEdgeIDs.Num() >= 3);
		TArrayView<const FEdgeID> TriEdges = TriangleEdges[TriangleID];
		for (int32 Index = 0; Index < 3; ++Index)
		{
			OutEdgeIDs[Index] = TriEdges[Index];
		}
	}

	/** Return the edges which form this triangle */
	TArrayView<FEdgeID> GetTriangleEdges(const FTriangleID TriangleID) const
	{
		return TriangleEdges[TriangleID];
	}

	/** Populates the passed array with adjacent triangles */
	template <typename Alloc>
	void GetTriangleAdjacentTriangles(const FTriangleID TriangleID, TArray<FTriangleID, Alloc>& OutTriangleIDs) const
	{
		OutTriangleIDs.Reset();
		for (const FEdgeID& EdgeID : GetTriangleEdges(TriangleID))
		{
			for (const int32& TriangleIndex : EdgeToTriangles.Find(EdgeID))
			{
				FTriangleID OtherTriangleID(TriangleIndex);
				if (OtherTriangleID != TriangleID)
				{
					OutTriangleIDs.Add(OtherTriangleID);
				}
			}
		}
	}

	/** Return adjacent triangles into a TArray with the specified allocator */
	template <typename Alloc>
	TArray<FTriangleID, Alloc> GetTriangleAdjacentTriangles(const FTriangleID TriangleID) const
	{
		TArray<FTriangleID, Alloc> Result;
		this->GetTriangleAdjacentTriangles(TriangleID, Result);
		return Result;
	}

	/** Return adjacent triangles to this triangle */
	TArray<FTriangleID> GetTriangleAdjacentTriangles(const FTriangleID TriangleID) const
	{
		TArray<FTriangleID> Result;
		this->GetTriangleAdjacentTriangles(TriangleID, Result);
		return Result;
	}

	/** Return UV indices for this triangle for the given channel */
	TArrayView<FUVID> GetTriangleUVIndices(const FTriangleID TriangleID, int32 UVChannel = 0) const
	{
		return TriangleUVs.Get(TriangleID, UVChannel);
	}

	/** Return the vertex instance which corresponds to the given vertex on the given triangle, or INDEX_NONE */
	FVertexInstanceID GetVertexInstanceForTriangleVertex(const FTriangleID TriangleID, const FVertexID VertexID) const
	{
		const FVertexInstanceID* VertexInstanceIDPtr = Algo::FindByPredicate(
			GetTriangleVertexInstances(TriangleID),
			[this, VertexID](const FVertexInstanceID VertexInstanceID) { return (GetVertexInstanceVertex(VertexInstanceID) == VertexID); });

		return VertexInstanceIDPtr ? *VertexInstanceIDPtr : FVertexInstanceID(INDEX_NONE);
	}

	/** Reverse the winding order of the vertices of this triangle */
	MESHDESCRIPTION_API void ReverseTriangleFacing(const FTriangleID TriangleID);

	/** Set the UV indices for this triangle */
	MESHDESCRIPTION_API void SetTriangleUVIndices(const FTriangleID TriangleID, TArrayView<const FUVID> UVIDs, int32 UVChannel = 0);

	//////////////////////////////////////////////////////////////////////
	// Polygon operations

	/** Return reference to an array of triangle IDs which comprise this polygon */
	TArrayView<const FTriangleID> GetPolygonTriangles(const FPolygonID PolygonID) const
	{
		return PolygonToTriangles.Find<FTriangleID>(PolygonID);
	}

	UE_DEPRECATED(4.26, "Please use GetPolygonTriangles() instead.")
	TArray<FTriangleID> GetPolygonTriangleIDs(const FPolygonID PolygonID) const
	{
		return TArray<FTriangleID>(GetPolygonTriangles(PolygonID));
	}

	/** Return the number of triangles which comprise this polygon */
	int32 GetNumPolygonTriangles(const FPolygonID PolygonID) const
	{
		return PolygonToTriangles.Find(PolygonID).Num();
	}

	/** Returns reference to an array of VertexInstance IDs forming the perimeter of this polygon */
	template <typename Alloc>
	void GetPolygonVertexInstances(const FPolygonID PolygonID, TArray<FVertexInstanceID, Alloc>& OutVertexInstanceIDs) const
	{
		OutVertexInstanceIDs.SetNumUninitialized(GetNumPolygonVertices(PolygonID));
		TArrayView<const FTriangleID> Tris = PolygonToTriangles.Find<FTriangleID>(PolygonID);
		if (Tris.Num() == 1)
		{
			OutVertexInstanceIDs = TriangleVertexInstances[Tris[0]];
		}
		else
		{
			TArray<TTuple<int32, int32>, TInlineAllocator<8>> Result;
			Result.SetNumUninitialized(Tris.Num() + 2);
			FindPolygonPerimeter(Tris, Result);
			for (int32 Index = 0; Index < Result.Num(); Index++)
			{
				FTriangleID TriangleID = Tris[Result[Index].Get<0>()];
				int32 VertexInstanceNumber = Result[Index].Get<1>();
				OutVertexInstanceIDs[Index] = TriangleVertexInstances[TriangleID][VertexInstanceNumber];
			}
		}
	}

	template <typename Alloc>
	TArray<FVertexInstanceID, Alloc> GetPolygonVertexInstances(const FPolygonID PolygonID) const
	{
		TArray<FVertexInstanceID, Alloc> Result;
		this->GetPolygonVertexInstances(PolygonID, Result);
		return Result;
	}

	TArray<FVertexInstanceID> GetPolygonVertexInstances(const FPolygonID PolygonID) const
	{
		TArray<FVertexInstanceID> Result;
		this->GetPolygonVertexInstances(PolygonID, Result);
		return Result;
	}

	/** Returns the number of vertices this polygon has */
	int32 GetNumPolygonVertices(const FPolygonID PolygonID) const
	{
		TArrayView<const FTriangleID> Tris = PolygonToTriangles.Find<FTriangleID>(PolygonID);
		return Tris.Num() + 2;
	}

	/** Populates the passed array of VertexIDs with the vertices which form the polygon perimeter */
	template <typename Alloc>
	void GetPolygonVertices(const FPolygonID PolygonID, TArray<FVertexID, Alloc>& OutVertexIDs) const
	{
		OutVertexIDs.SetNumUninitialized(GetNumPolygonVertices(PolygonID));
		TArrayView<const FTriangleID> Tris = PolygonToTriangles.Find<FTriangleID>(PolygonID);
		if (Tris.Num() == 1)
		{
			OutVertexIDs = TriangleVertices[Tris[0]];
		}
		else
		{
			TArray<TTuple<int32, int32>, TInlineAllocator<8>> Result;
			Result.SetNumUninitialized(Tris.Num() + 2);
			FindPolygonPerimeter(Tris, Result);
			for (int32 Index = 0; Index < Result.Num(); Index++)
			{
				FTriangleID TriangleID = Tris[Result[Index].Get<0>()];
				int32 VertexNumber = Result[Index].Get<1>();
				OutVertexIDs[Index] = TriangleVertices[TriangleID][VertexNumber];
			}
		}
	}

	/** Returns the vertices which form the polygon perimeter as an array templated on the given allocator */
	template <typename Alloc>
	TArray<FVertexID, Alloc> GetPolygonVertices(const FPolygonID PolygonID) const
	{
		TArray<FVertexID, Alloc> Result;
		this->GetPolygonVertices(PolygonID, Result);
		return Result;
	}

	/** Returns the vertices which form the polygon perimeter */
	TArray<FVertexID> GetPolygonVertices(const FPolygonID PolygonID) const
	{
		TArray<FVertexID> Result;
		this->GetPolygonVertices(PolygonID, Result);
		return Result;
	}

	/** Populates the passed array with the edges which form the polygon perimeter */
	template <typename Alloc>
	void GetPolygonPerimeterEdges(const FPolygonID PolygonID, TArray<FEdgeID, Alloc>& OutEdgeIDs) const
	{
		OutEdgeIDs.SetNumUninitialized(GetNumPolygonVertices(PolygonID));
		TArrayView<const FTriangleID> Tris = PolygonToTriangles.Find<FTriangleID>(PolygonID);
		if (Tris.Num() == 1)
		{
			OutEdgeIDs = TriangleEdges[Tris[0]];
		}
		else
		{
			TArray<TTuple<int32, int32>, TInlineAllocator<8>> Result;
			Result.SetNumUninitialized(Tris.Num() + 2);
			FindPolygonPerimeter(Tris, Result);
			for (int32 Index = 0; Index < Result.Num(); Index++)
			{
				FTriangleID TriangleID = Tris[Result[Index].Get<0>()];
				int32 EdgeNumber = Result[Index].Get<1>();
				OutEdgeIDs[Index] = TriangleEdges[TriangleID][EdgeNumber];
			}
		}
	}

	/** Returns the vertices which form the polygon perimeter as an array templated on the given allocator */
	template <typename Alloc>
	TArray<FEdgeID, Alloc> GetPolygonPerimeterEdges(const FPolygonID PolygonID) const
	{
		TArray<FEdgeID, Alloc> Result;
		this->GetPolygonPerimeterEdges(PolygonID, Result);
		return Result;
	}

	/** Returns the vertices which form the polygon perimeter */
	TArray<FEdgeID> GetPolygonPerimeterEdges(const FPolygonID PolygonID) const
	{
		TArray<FEdgeID> Result;
		this->GetPolygonPerimeterEdges(PolygonID, Result);
		return Result;
	}

	/** Populate the provided array with a list of edges which are internal to the polygon, i.e. those which separate
	    constituent triangles. */
	template <typename Alloc>
	void GetPolygonInternalEdges(const FPolygonID PolygonID, TArray<FEdgeID, Alloc>& OutEdgeIDs) const
	{
		OutEdgeIDs.Reset(GetNumPolygonVertices(PolygonID) - 3);
		if (GetNumPolygonVertices(PolygonID) > 3)
		{
			TArray<FVertexInstanceID> VertexInstanceIDs = GetPolygonVertexInstances(PolygonID);
			for (const FVertexInstanceID& VertexInstanceID : VertexInstanceIDs)
			{
				for (const FEdgeID& EdgeID : GetVertexConnectedEdgeIDs(GetVertexInstanceVertex(VertexInstanceID)))
				{
					if (!OutEdgeIDs.Contains(EdgeID) && IsEdgeInternalToPolygon(EdgeID, PolygonID))
					{
						OutEdgeIDs.Add(EdgeID);
					}
				}
			}
		}
	}

	/** Return the internal edges of this polygon, i.e. those which separate constituent triangles */
	template <typename Alloc>
	TArray<FEdgeID, Alloc> GetPolygonInternalEdges(const FPolygonID PolygonID) const
	{
		TArray<FEdgeID, Alloc> Result;
		this->GetPolygonInternalEdges(PolygonID, Result);
		return Result;
	}

	/** Return the internal edges of this polygon, i.e. those which separate constituent triangles */
	TArray<FEdgeID> GetPolygonInternalEdges(const FPolygonID PolygonID) const
	{
		TArray<FEdgeID> Result;
		this->GetPolygonInternalEdges(PolygonID, Result);
		return Result;
	}

	/** Return the number of internal edges in this polygon */
	int32 GetNumPolygonInternalEdges(const FPolygonID PolygonID) const
	{
		return PolygonToTriangles.Find(PolygonID).Num() - 1;
	}

	/** Populates the passed array with adjacent polygons */
	template <typename Alloc>
	void GetPolygonAdjacentPolygons(const FPolygonID PolygonID, TArray<FPolygonID, Alloc>& OutPolygonIDs) const
	{
		OutPolygonIDs.Reset();
		for (const FEdgeID& EdgeID : GetPolygonPerimeterEdges<TInlineAllocator<16>>(PolygonID))
		{
			for (const FPolygonID& OtherPolygonID : GetEdgeConnectedPolygons<TInlineAllocator<8>>(EdgeID))
			{
				if (OtherPolygonID != PolygonID)
				{
					OutPolygonIDs.Add(OtherPolygonID);
				}
			}
		}
	}

	/** Return adjacent polygons into a TArray with the specified allocator */
	template <typename Alloc>
	TArray<FPolygonID, Alloc> GetPolygonAdjacentPolygons(const FPolygonID PolygonID) const
	{
		TArray<FPolygonID, Alloc> Result;
		this->GetPolygonAdjacentPolygons(PolygonID, Result);
		return Result;
	}

	/** Return adjacent polygons to this polygon */
	TArray<FPolygonID> GetPolygonAdjacentPolygons(const FPolygonID PolygonID) const
	{
		TArray<FPolygonID> Result;
		this->GetPolygonAdjacentPolygons(PolygonID, Result);
		return Result;
	}

	/** Return the polygon group associated with a polygon */
	FPolygonGroupID GetPolygonPolygonGroup(const FPolygonID PolygonID) const
	{
		return PolygonPolygonGroups[PolygonID];
	}

	/** Return the vertex instance which corresponds to the given vertex on the given polygon, or INDEX_NONE */
	FVertexInstanceID GetVertexInstanceForPolygonVertex(const FPolygonID PolygonID, const FVertexID VertexID) const
	{
		TArray<FVertexInstanceID> VertexInstanceIDs = GetPolygonVertexInstances(PolygonID);
		const FVertexInstanceID* VertexInstanceIDPtr = VertexInstanceIDs.FindByPredicate(
			[this, VertexID](const FVertexInstanceID VertexInstanceID) { return (GetVertexInstanceVertex(VertexInstanceID) == VertexID); });

		return VertexInstanceIDPtr ? *VertexInstanceIDPtr : FVertexInstanceID(INDEX_NONE);
	}

	/** Set the vertex instance at the given index around the polygon to the new value */
	MESHDESCRIPTION_API void SetPolygonVertexInstance(const FPolygonID PolygonID, const int32 PerimeterIndex, const FVertexInstanceID VertexInstanceID);
	MESHDESCRIPTION_API void SetPolygonVertexInstances(const FPolygonID PolygonID, TArrayView<const FVertexInstanceID> VertexInstanceIDs);

	/** Sets the polygon group associated with a polygon */
	void SetPolygonPolygonGroup(const FPolygonID PolygonID, const FPolygonGroupID PolygonGroupID)
	{
		FPolygonGroupID OldPolygonGroupID = PolygonPolygonGroups[PolygonID];
		PolygonGroupToPolygons.RemoveReferenceFromKey(OldPolygonGroupID, PolygonID);

		PolygonPolygonGroups[PolygonID] = PolygonGroupID;
		PolygonGroupToPolygons.AddReferenceToKey(PolygonGroupID, PolygonID);

		for (const int32& TriangleIndex: PolygonToTriangles.Find(PolygonID))
		{
			const FTriangleID TriangleID(TriangleIndex);
			check(TrianglePolygonGroups[TriangleID] == OldPolygonGroupID);
			PolygonGroupToTriangles.RemoveReferenceFromKey(OldPolygonGroupID, TriangleID);

			TrianglePolygonGroups[TriangleID] = PolygonGroupID;
			PolygonGroupToTriangles.AddReferenceToKey(PolygonGroupID, TriangleID);
		}
	}

	/** Reverse the winding order of the vertices of this polygon */
	MESHDESCRIPTION_API void ReversePolygonFacing(const FPolygonID PolygonID);

	/** Determines the vertex instances which form the perimeter of a polygon */
	MESHDESCRIPTION_API void FindPolygonPerimeter(TArrayView<const FTriangleID> Triangles, TArrayView<TTuple<int32, int32>> Result) const;
	MESHDESCRIPTION_API void FindPolygonPerimeter(const FPolygonID PolygonID, TArrayView<FEdgeID> Edges) const;

	/** Generates triangles and internal edges for the given polygon */
	MESHDESCRIPTION_API void ComputePolygonTriangulation(const FPolygonID PolygonID);

	MESHDESCRIPTION_API void SplitPolygon(FPolygonID PolygonID);


	//////////////////////////////////////////////////////////////////////
	// Polygon group operations

	/** Returns the polygons associated with the given polygon group */
	TArrayView<const FPolygonID> GetPolygonGroupPolygonIDs(const FPolygonGroupID PolygonGroupID) const
	{
		return PolygonGroupToPolygons.Find<FPolygonID>(PolygonGroupID);
	}

	UE_DEPRECATED(4.26, "Please use GetPolygonGroupPolygonIDs() instead.")
	TArray<FPolygonID> GetPolygonGroupPolygons(const FPolygonGroupID PolygonGroupID) const
	{
		return TArray<FPolygonID>(GetPolygonGroupPolygonIDs(PolygonGroupID));
	}

	/** Returns the triangles associated with the given polygon group */
	TArrayView<const FTriangleID> GetPolygonGroupTriangles(const FPolygonGroupID PolygonGroupID) const
	{
		return PolygonGroupToTriangles.Find<FTriangleID>(PolygonGroupID);
	}

	/** Returns the number of polygons in this polygon group */
	int32 GetNumPolygonGroupPolygons(const FPolygonGroupID PolygonGroupID) const
	{
		return PolygonGroupToPolygons.Find(PolygonGroupID).Num();
	}

	/** Returns the number of triangles in this polygon group */
	int32 GetNumPolygonGroupTriangles(const FPolygonGroupID PolygonGroupID) const
	{
		return PolygonGroupToTriangles.Find(PolygonGroupID).Num();
	}

	/** Remaps polygon groups according to the supplied map */
	MESHDESCRIPTION_API void RemapPolygonGroups(const TMap<FPolygonGroupID, FPolygonGroupID>& Remap);
	
	/** Transfer the source polygon group data to the destination polygon group. The source data is append to the destination */
	MESHDESCRIPTION_API void TransferPolygonGroup(FPolygonGroupID SourceID, FPolygonGroupID DestinationID);

	//////////////////////////////////////////////////////////////////////
	// Whole mesh operations

	/** Determines if calling Compact() would perform actual compaction or not. */
	MESHDESCRIPTION_API bool NeedsCompact() const;

	/** Compacts the data held in the mesh description, and returns an object describing how the IDs have been remapped. */
	MESHDESCRIPTION_API void Compact( FElementIDRemappings& OutRemappings );

	/** Remaps the element IDs in the mesh description according to the passed in object */
	MESHDESCRIPTION_API void Remap( const FElementIDRemappings& Remappings );

	/** Gets the number of UV element channels */
	int32 GetNumUVElementChannels() const { return UVElements->GetNumChannels(); }

	/** Sets the specified number of UV channels */
	MESHDESCRIPTION_API void SetNumUVChannels(const int32 NumUVChannels);

	/** Returns bounds of vertices */
	MESHDESCRIPTION_API FBoxSphereBounds GetBounds() const;

	/** Retriangulates the entire mesh */
	MESHDESCRIPTION_API void TriangulateMesh();

	/** Reverses the winding order of all polygons in the mesh */
	MESHDESCRIPTION_API void ReverseAllPolygonFacing();

	MESHDESCRIPTION_API float GetTriangleCornerAngleForVertex(const FTriangleID TriangleID, const FVertexID VertexID) const;
	MESHDESCRIPTION_API float GetPolygonCornerAngleForVertex(const FPolygonID PolygonID, const FVertexID VertexID) const;

	MESHDESCRIPTION_API FBox ComputeBoundingBox() const;

private:

	MESHDESCRIPTION_API FPlane ComputePolygonPlane(TArrayView<const FVertexInstanceID> PerimeterVertexInstanceIDs) const;
	MESHDESCRIPTION_API FVector ComputePolygonNormal(TArrayView<const FVertexInstanceID> PerimeterVertexInstanceIDs) const;

private:

	MESHDESCRIPTION_API void Initialize();
	MESHDESCRIPTION_API void InitializeIndexers();
	MESHDESCRIPTION_API void Cache();

	MESHDESCRIPTION_API void CreateVertexInstance_Internal(const FVertexInstanceID VertexInstanceID, const FVertexID VertexID);
	MESHDESCRIPTION_API void CreateEdge_Internal(const FEdgeID EdgeID, const FVertexID VertexID0, const FVertexID VertexID1);
	MESHDESCRIPTION_API void CreateTriangle_Internal(const FTriangleID TriangleID, const FPolygonGroupID PolygonGroupID, TArrayView<const FVertexInstanceID> VertexInstanceIDs, TArray<FEdgeID>* OutEdgeIDs);
	MESHDESCRIPTION_API void CreatePolygon_Internal(const FPolygonID PolygonID, const FPolygonGroupID PolygonGroupID, TArrayView<const FVertexInstanceID> VertexInstanceIDs, TArray<FEdgeID>* OutEdgeIDs);
	MESHDESCRIPTION_API void CreatePolygonTriangles(const FPolygonID PolygonID, TArrayView<const FVertexInstanceID> VertexInstanceIDs);
	MESHDESCRIPTION_API void RemovePolygonTriangles(const FPolygonID PolygonID);

	template <template <typename...> class TContainer>
	void DeleteVertexInstance_Internal(const FVertexInstanceID VertexInstanceID, TContainer<FVertexID>* InOutOrphanedVerticesPtr = nullptr);
	template <template <typename...> class TContainer>
	void DeleteEdge_Internal(const FEdgeID EdgeID, TContainer<FVertexID>* InOutOrphanedVerticesPtr = nullptr);
	template <template <typename...> class TContainer>
	void DeleteTriangle_Internal(const FTriangleID TriangleID, TContainer<FEdgeID>* InOutOrphanedEdgesPtr = nullptr, TContainer<FVertexInstanceID>* InOutOrphanedVertexInstancesPtr = nullptr, TContainer<FPolygonGroupID>* InOutOrphanedPolygonGroupsPtr = nullptr);
	template <template <typename...> class TContainer>
	void DeletePolygon_Internal(const FPolygonID PolygonID, TContainer<FEdgeID>* InOutOrphanedEdgesPtr = nullptr, TContainer<FVertexInstanceID>* InOutOrphanedVertexInstancesPtr = nullptr, TContainer<FPolygonGroupID>* InOutOrphanedPolygonGroupsPtr = nullptr);

	/** Given a set of index remappings, fixes up references to element IDs */
	MESHDESCRIPTION_API void FixUpElementIDs(const FElementIDRemappings& Remappings);

	template <class T>
	void AddUnique(TArray<T>& Container, const T& Item)
	{
		Container.AddUnique(Item);
	}

	template <class T>
	void AddUnique(TSet<T>& Container, const T& Item)
	{
		Container.Add(Item);
	}

public:
	static MESHDESCRIPTION_API FName VerticesName;
	static MESHDESCRIPTION_API FName VertexInstancesName;
	static MESHDESCRIPTION_API FName UVsName;
	static MESHDESCRIPTION_API FName EdgesName;
	static MESHDESCRIPTION_API FName TrianglesName;
	static MESHDESCRIPTION_API FName PolygonsName;
	static MESHDESCRIPTION_API FName PolygonGroupsName;

private:
	TMap<FName, FMeshElementTypeWrapper> Elements;

	FMeshElementChannels* VertexElements;
	FMeshElementChannels* VertexInstanceElements;
	FMeshElementChannels* UVElements;
	FMeshElementChannels* EdgeElements;
	FMeshElementChannels* TriangleElements;
	FMeshElementChannels* PolygonElements;
	FMeshElementChannels* PolygonGroupElements;

	TMeshAttributesArray<FVertexID> VertexInstanceVertices;
	TMeshAttributesArray<TArrayView<FVertexID>> EdgeVertices;
	TMeshAttributesArray<TArrayView<FVertexInstanceID>> TriangleVertexInstances;
	TMeshAttributesArray<FPolygonID> TrianglePolygons;
	TMeshAttributesArray<TArrayView<FEdgeID>> TriangleEdges;
	TMeshAttributesArray<TArrayView<FVertexID>> TriangleVertices;
	TMeshAttributesArray<TArrayView<FUVID>> TriangleUVs;
	TMeshAttributesArray<FPolygonGroupID> TrianglePolygonGroups;
	TMeshAttributesArray<FPolygonGroupID> PolygonPolygonGroups;
	TMeshAttributesArray<FVector3f> VertexPositions;

	// The indexers are all mutable because they may be rebuilt during what is essentially a get operation
	mutable FMeshElementIndexer VertexToVertexInstances;
	mutable FMeshElementIndexer VertexToEdges;
	mutable FMeshElementIndexer VertexInstanceToTriangles;
	mutable FMeshElementIndexer EdgeToTriangles;
	mutable FMeshElementIndexer UVToTriangles;
	mutable FMeshElementIndexer PolygonToTriangles;
	mutable FMeshElementIndexer PolygonGroupToTriangles;
	mutable FMeshElementIndexer PolygonGroupToPolygons;
	
	friend struct FMeshDescriptionBulkData; 
};

#if WITH_EDITORONLY_DATA

/**
 * Bulk data storage for FMeshDescription
 */
struct FMeshDescriptionBulkData
{
public:
	FMeshDescriptionBulkData() = default;

	/** Serialization */
	MESHDESCRIPTION_API void Serialize(FArchive& Ar, UObject* Owner);

	/** Store a new mesh description in the bulk data */
	MESHDESCRIPTION_API void SaveMeshDescription(FMeshDescription& MeshDescription);

	/** Load the mesh description from the bulk data */
	MESHDESCRIPTION_API void LoadMeshDescription(FMeshDescription& MeshDescription);

	/** Empties the bulk data */
	MESHDESCRIPTION_API void Empty();

	/** Returns true if there is no bulk data available */
	bool IsEmpty() const { return !BulkData.HasPayloadData(); }

	/** Return unique ID string for this bulk data that does not include versioning info. 
	 *  Use this for constructing IDs that do not rely on the serialization format of 
	 *  FMeshDescription but is only dependent on its contents, such as storing intermediate
	 *  representations build from the FMeshDescription data in the DDC.
	 */
	FString GetUnversionedIdString() const
	{
		return GetIdStringFromVersionList(GetFrozenVersions());
	}

	/** Return unique ID string for this bulk data that also includes current versioning info for
	 *  FMeshDescription serialization. 
	 *  Use this for constructing IDs that are required for serializing the FMeshDescription
	 *  bulk data to disk, such as storing FMeshDescription archives in a shared DDC storage.
	 */
	FString GetVersionedIdString() const
	{
		return GetIdStringFromVersionList(FMeshDescription::GetSerializationFormatVersions());
	}

	UE_DEPRECATED(5.8, "Use GetVersionedIdString or GetUnversionedIdString instead, depending on requirements")
	FString GetIdString() const
	{
		return GetUnversionedIdString();
	}
	
	/** Return an ID string that represents the serialization version for FMeshDescription based on the
	 *  live custom versions. This differs from GetVersionedIdString() in that it does not include
	 *  the content hash.
	 */
	static MESHDESCRIPTION_API FString GetSerializationVersionIdString();
	
	/** Uses a hash as the GUID, useful to prevent recomputing content already in cache. */
	MESHDESCRIPTION_API void UseHashAsGuid();

	/** Gets the size of the serialized bulk data */
	int64 GetBulkDataSize() { return BulkData.GetPayloadSize(); }

	/** Returns the CustomVersions GUIDs that are used when serializing an FMeshDescription. */
	static MESHDESCRIPTION_API TConstArrayView<FGuid> GetMeshDescriptionCustomVersions();
	
private:
	void UpdateMeshDescriptionFormat();
	
	MESHDESCRIPTION_API FString GetIdStringFromVersionList(TConstArrayView<TPair<FGuid, int32>> InVersionList) const;

	static MESHDESCRIPTION_API TConstArrayView<TPair<FGuid, int32>> GetFrozenVersions();
	
#if WITH_EDITOR
	/** Protects simultaneous access to BulkData */
	FRWLock       BulkDataLock;
#endif
	/** Internally store bulk data as bytes */
	UE::Serialization::FEditorBulkData BulkData;

	/** GUID associated with the data stored herein. */
	FGuid Guid;

	/** Gets this bulk data hash */
	MESHDESCRIPTION_API FGuid GetHash() const;

	/** Take a copy of the bulk data versioning so it can be propagated to the bulk data reader when deserializing MeshDescription */
	FCustomVersionContainer CustomVersions;
	FPackageFileVersion UEVersion;
	int32 LicenseeUEVersion = INDEX_NONE;


	/** Whether the bulk data has been written via SaveMeshDescription */
	bool bBulkDataUpdated = false;

	/** Uses hash instead of guid to identify content to improve DDC cache hit. */
	bool bGuidIsHash = false;
};
#endif // WITH_EDITORONLY_DATA

