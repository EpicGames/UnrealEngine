// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/UnrealMathSSE.h"
#include "MeshAttributes.h"
#include "MeshDescription.h"
#include "MeshTypes.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MeshDescriptionBase.generated.h"

class FArchive;
struct FFrame;
template <typename ElementIDType> class TAttributesSet;



UCLASS(BlueprintType, MinimalAPI)
class UMeshDescriptionBase : public UObject
{
	GENERATED_BODY()

public:
	// UObject interface
	MESHDESCRIPTION_API virtual void Serialize(FArchive& Ar) override;
	//

	/** Get a reference to the actual mesh description */
	FMeshDescription& GetMeshDescription()
	{
		return OwnedMeshDescription;
	}

	const FMeshDescription& GetMeshDescription() const
	{
		return OwnedMeshDescription;
	}

	/** Set the mesh description */
	void SetMeshDescription(FMeshDescription InMeshDescription)
	{ 
		OwnedMeshDescription = MoveTemp(InMeshDescription);
	}

	// UMeshDescriptionBase interface
	MESHDESCRIPTION_API virtual void RegisterAttributes();
	virtual FMeshAttributes& GetRequiredAttributes() { return *RequiredAttributes; }
	virtual const FMeshAttributes& GetRequiredAttributes() const { return *RequiredAttributes; }
	//

	/** Reset the contained mesh description */
	MESHDESCRIPTION_API void Reset();

	/** Accessors for mesh element arrays */
	FVertexArray& Vertices() { return GetMeshDescription().Vertices(); }
	const FVertexArray& Vertices() const { return GetMeshDescription().Vertices(); }

	FVertexInstanceArray& VertexInstances() { return GetMeshDescription().VertexInstances(); }
	const FVertexInstanceArray& VertexInstances() const { return GetMeshDescription().VertexInstances(); }

	FEdgeArray& Edges() { return GetMeshDescription().Edges(); }
	const FEdgeArray& Edges() const { return GetMeshDescription().Edges(); }

	FTriangleArray& Triangles() { return GetMeshDescription().Triangles(); }
	const FTriangleArray& Triangles() const { return GetMeshDescription().Triangles(); }

	FPolygonArray& Polygons() { return GetMeshDescription().Polygons(); }
	const FPolygonArray& Polygons() const { return GetMeshDescription().Polygons(); }

	FPolygonGroupArray& PolygonGroups() { return GetMeshDescription().PolygonGroups(); }
	const FPolygonGroupArray& PolygonGroups() const { return GetMeshDescription().PolygonGroups(); }

	/** Accessors for mesh element attributes */
	TAttributesSet<FVertexID>& VertexAttributes() { return GetMeshDescription().VertexAttributes(); }
	const TAttributesSet<FVertexID>& VertexAttributes() const { return GetMeshDescription().VertexAttributes(); }

	TAttributesSet<FVertexInstanceID>& VertexInstanceAttributes() { return GetMeshDescription().VertexInstanceAttributes(); }
	const TAttributesSet<FVertexInstanceID>& VertexInstanceAttributes() const { return GetMeshDescription().VertexInstanceAttributes(); }

	TAttributesSet<FEdgeID>& EdgeAttributes() { return GetMeshDescription().EdgeAttributes(); }
	const TAttributesSet<FEdgeID>& EdgeAttributes() const { return GetMeshDescription().EdgeAttributes(); }

	TAttributesSet<FTriangleID>& TriangleAttributes() { return GetMeshDescription().TriangleAttributes(); }
	const TAttributesSet<FTriangleID>& TriangleAttributes() const { return GetMeshDescription().TriangleAttributes(); }

	TAttributesSet<FPolygonID>& PolygonAttributes() { return GetMeshDescription().PolygonAttributes(); }
	const TAttributesSet<FPolygonID>& PolygonAttributes() const { return GetMeshDescription().PolygonAttributes(); }

	TAttributesSet<FPolygonGroupID>& PolygonGroupAttributes() { return GetMeshDescription().PolygonGroupAttributes(); }
	const TAttributesSet<FPolygonGroupID>& PolygonGroupAttributes() const { return GetMeshDescription().PolygonGroupAttributes(); }

	/** Accessors for cached vertex position array */
	TVertexAttributesRef<FVector3f> GetVertexPositions() { return GetRequiredAttributes().GetVertexPositions(); }
	TVertexAttributesConstRef<FVector3f> GetVertexPositions() const { return GetRequiredAttributes().GetVertexPositions(); }

public:

	/** Empty the mesh description */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void Empty();

	/** Return whether the mesh description is empty */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API bool IsEmpty() const;


	///////////////////////////////////////////////////////
	// Create / remove / count mesh elements

	/** Reserves space for this number of new vertices */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void ReserveNewVertices(int32 NumberOfNewVertices);

	/** Adds a new vertex to the mesh and returns its ID */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API FVertexID CreateVertex();

	/** Adds a new vertex to the mesh with the given ID */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void CreateVertexWithID(FVertexID VertexID);

	/** Deletes a vertex from the mesh */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void DeleteVertex(FVertexID VertexID);

	/** Returns whether the passed vertex ID is valid */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API bool IsVertexValid(FVertexID VertexID) const;

	/** Returns the number of vertices */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	int32 GetVertexCount() const { return Vertices().Num(); }

	/** Reserves space for this number of new vertex instances */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void ReserveNewVertexInstances(int32 NumberOfNewVertexInstances);

	/** Adds a new vertex instance to the mesh and returns its ID */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API FVertexInstanceID CreateVertexInstance(FVertexID VertexID);

	/** Adds a new vertex instance to the mesh with the given ID */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void CreateVertexInstanceWithID(FVertexInstanceID VertexInstanceID, FVertexID VertexID);

	/** Deletes a vertex instance from a mesh */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void DeleteVertexInstance(FVertexInstanceID VertexInstanceID, TArray<FVertexID>& OrphanedVertices);

	/** Returns whether the passed vertex instance ID is valid */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API bool IsVertexInstanceValid(FVertexInstanceID VertexInstanceID) const;

	/** Returns the number of vertex instances */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	int32 GetVertexInstanceCount() const { return VertexInstances().Num(); }

	/** Reserves space for this number of new edges */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void ReserveNewEdges(int32 NumberOfNewEdges);

	/** Adds a new edge to the mesh and returns its ID */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API FEdgeID CreateEdge(FVertexID VertexID0, FVertexID VertexID1);

	/** Adds a new edge to the mesh with the given ID */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void CreateEdgeWithID(FEdgeID EdgeID, FVertexID VertexID0, FVertexID VertexID1);

	/** Deletes an edge from a mesh */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void DeleteEdge(FEdgeID EdgeID, TArray<FVertexID>& OrphanedVertices);

	/** Returns whether the passed edge ID is valid */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API bool IsEdgeValid(FEdgeID EdgeID) const;

	/** Returns the number of edges */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	int32 GetEdgeCount() const { return Edges().Num(); }

	/** Reserves space for this number of new triangles */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void ReserveNewTriangles(int32 NumberOfNewTriangles);

	/** Adds a new triangle to the mesh and returns its ID. This will also make an encapsulating polygon, and any missing edges. */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API FTriangleID CreateTriangle(FPolygonGroupID PolygonGroupID, const TArray<FVertexInstanceID>& VertexInstanceIDs, TArray<FEdgeID>& NewEdgeIDs);

	/** Adds a new triangle to the mesh with the given ID. This will also make an encapsulating polygon, and any missing edges. */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void CreateTriangleWithID(FTriangleID TriangleID, FPolygonGroupID PolygonGroupID, const TArray<FVertexInstanceID>& VertexInstanceIDs, TArray<FEdgeID>& NewEdgeIDs);

	/** Deletes a triangle from the mesh */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void DeleteTriangle(FTriangleID TriangleID, TArray<FEdgeID>& OrphanedEdges, TArray<FVertexInstanceID>& OrphanedVertexInstances, TArray<FPolygonGroupID>& OrphanedPolygonGroupsPtr);

	/** Returns whether the passed triangle ID is valid */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API bool IsTriangleValid(const FTriangleID TriangleID) const;

	/** Returns the number of triangles */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	int32 GetTriangleCount() const { return Triangles().Num(); }

	/** Reserves space for this number of new polygons */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void ReserveNewPolygons(const int32 NumberOfNewPolygons);

	/** Adds a new polygon to the mesh and returns its ID. This will also make any missing edges, and all constituent triangles. */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API FPolygonID CreatePolygon(FPolygonGroupID PolygonGroupID, TArray<FVertexInstanceID>& VertexInstanceIDs, TArray<FEdgeID>& NewEdgeIDs);

	/** Adds a new polygon to the mesh with the given ID. This will also make any missing edges, and all constituent triangles. */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void CreatePolygonWithID(FPolygonID PolygonID, FPolygonGroupID PolygonGroupID, TArray<FVertexInstanceID>& VertexInstanceIDs, TArray<FEdgeID>& NewEdgeIDs);

	/** Deletes a polygon from the mesh */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void DeletePolygon(FPolygonID PolygonID, TArray<FEdgeID>& OrphanedEdges, TArray<FVertexInstanceID>& OrphanedVertexInstances, TArray<FPolygonGroupID>& OrphanedPolygonGroups);

	/** Returns whether the passed polygon ID is valid */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API bool IsPolygonValid(FPolygonID PolygonID) const;

	/** Returns the number of polygons */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	int32 GetPolygonCount() const { return Polygons().Num(); }

	/** Reserves space for this number of new polygon groups */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void ReserveNewPolygonGroups(int32 NumberOfNewPolygonGroups);

	/** Adds a new polygon group to the mesh and returns its ID */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API FPolygonGroupID CreatePolygonGroup();

	/** Adds a new polygon group to the mesh with the given ID */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void CreatePolygonGroupWithID(FPolygonGroupID PolygonGroupID);

	/** Deletes a polygon group from the mesh */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void DeletePolygonGroup(FPolygonGroupID PolygonGroupID);

	/** Returns whether the passed polygon group ID is valid */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API bool IsPolygonGroupValid(FPolygonGroupID PolygonGroupID) const;

	/** Returns the number of polygon groups */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	int32 GetPolygonGroupCount() const { return PolygonGroups().Num(); }


	//////////////////////////////////////////////////////////////////////
	// Vertex operations

	/** Returns whether a given vertex is orphaned, i.e. it doesn't form part of any polygon */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API bool IsVertexOrphaned(FVertexID VertexID) const;

	/** Returns the edge ID defined by the two given vertex IDs, if there is one; otherwise INDEX_NONE */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API FEdgeID GetVertexPairEdge(FVertexID VertexID0, FVertexID VertexID1) const;

	/** Returns reference to an array of Edge IDs connected to this vertex */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetVertexConnectedEdges(FVertexID VertexID, TArray<FEdgeID>& OutEdgeIDs) const;

	/** Returns number of edges connected to this vertex */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumVertexConnectedEdges(FVertexID VertexID) const;

	/** Returns reference to an array of VertexInstance IDs instanced from this vertex */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetVertexVertexInstances(FVertexID VertexID, TArray<FVertexInstanceID>& OutVertexInstanceIDs) const;

	/** Returns number of vertex instances created from this vertex */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumVertexVertexInstances(FVertexID VertexID) const;

	/** Returns the triangles connected to this vertex */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetVertexConnectedTriangles(FVertexID VertexID, TArray<FTriangleID>& OutConnectedTriangleIDs) const;

	/** Returns number of triangles connected to this vertex */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumVertexConnectedTriangles(FVertexID VertexID) const;

	/** Returns the polygons connected to this vertex */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetVertexConnectedPolygons(FVertexID VertexID, TArray<FPolygonID>& OutConnectedPolygonIDs) const;

	/** Returns the number of polygons connected to this vertex */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumVertexConnectedPolygons(FVertexID VertexID) const;

	/** Returns the vertices adjacent to this vertex */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetVertexAdjacentVertices(FVertexID VertexID, TArray<FVertexID>& OutAdjacentVertexIDs) const;

	/** Gets a vertex position */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API FVector GetVertexPosition(FVertexID VertexID) const;

	/** Sets a vertex position */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void SetVertexPosition(FVertexID VertexID, const FVector& Position);


	//////////////////////////////////////////////////////////////////////
	// Vertex instance operations

	/** Returns the vertex ID associated with the given vertex instance */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API FVertexID GetVertexInstanceVertex(FVertexInstanceID VertexInstanceID) const;

	/** Returns the edge ID defined by the two given vertex instance IDs, if there is one; otherwise INDEX_NONE */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API FEdgeID GetVertexInstancePairEdge(FVertexInstanceID VertexInstanceID0, FVertexInstanceID VertexInstanceID1) const;

	/** Returns reference to an array of Triangle IDs connected to this vertex instance */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetVertexInstanceConnectedTriangles(FVertexInstanceID VertexInstanceID, TArray<FTriangleID>& OutConnectedTriangleIDs) const;

	/** Returns the number of triangles connected to this vertex instance */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumVertexInstanceConnectedTriangles(FVertexInstanceID VertexInstanceID) const;

	/** Returns the polygons connected to this vertex instance */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetVertexInstanceConnectedPolygons(FVertexInstanceID VertexInstanceID, TArray<FPolygonID>& OutConnectedPolygonIDs) const;

	/** Returns the number of polygons connected to this vertex instance. */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumVertexInstanceConnectedPolygons(FVertexInstanceID VertexInstanceID) const;


	//////////////////////////////////////////////////////////////////////
	// Edge operations

	/** Determine whether a given edge is an internal edge between triangles of a polygon */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API bool IsEdgeInternal(FEdgeID EdgeID) const;

	/** Determine whether a given edge is an internal edge between triangles of a specific polygon */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API bool IsEdgeInternalToPolygon(FEdgeID EdgeID, FPolygonID PolygonID) const;

	/** Returns reference to an array of triangle IDs connected to this edge */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetEdgeConnectedTriangles(FEdgeID EdgeID, TArray<FTriangleID>& OutConnectedTriangleIDs) const;

	/** Returns the number of triangles connected to this edge */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumEdgeConnectedTriangles(FEdgeID EdgeID) const;

	/** Returns the polygons connected to this edge */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetEdgeConnectedPolygons(FEdgeID EdgeID, TArray<FPolygonID>& OutConnectedPolygonIDs) const;

	/** Returns the number of polygons connected to this edge */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumEdgeConnectedPolygons(FEdgeID EdgeID) const;

	/** Returns the vertex ID corresponding to one of the edge endpoints */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API FVertexID GetEdgeVertex(FEdgeID EdgeID, int32 VertexNumber) const;

	/** Returns a pair of vertex IDs defining the edge */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetEdgeVertices(const FEdgeID EdgeID, TArray<FVertexID>& OutVertexIDs) const;


	//////////////////////////////////////////////////////////////////////
	// Triangle operations

	/** Get the polygon which contains this triangle */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API FPolygonID GetTrianglePolygon(FTriangleID TriangleID) const;

	/** Get the polygon group which contains this triangle */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API FPolygonGroupID GetTrianglePolygonGroup(FTriangleID TriangleID) const;

	/** Determines if this triangle is part of an n-gon */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API bool IsTrianglePartOfNgon(FTriangleID TriangleID) const;

	/** Get the vertex instances which define this triangle */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetTriangleVertexInstances(FTriangleID TriangleID, TArray<FVertexInstanceID>& OutVertexInstanceIDs) const;

	/** Get the specified vertex instance by index */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API FVertexInstanceID GetTriangleVertexInstance(FTriangleID TriangleID, int32 Index) const;

	/** Returns the vertices which define this triangle */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetTriangleVertices(FTriangleID TriangleID, TArray<FVertexID>& OutVertexIDs) const;

	/** Returns the edges which define this triangle */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetTriangleEdges(FTriangleID TriangleID, TArray<FEdgeID>& OutEdgeIDs) const;

	/** Returns the adjacent triangles to this triangle */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetTriangleAdjacentTriangles(FTriangleID TriangleID, TArray<FTriangleID>& OutTriangleIDs) const;

	/** Return the vertex instance which corresponds to the given vertex on the given triangle, or INDEX_NONE */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API FVertexInstanceID GetVertexInstanceForTriangleVertex(FTriangleID TriangleID, FVertexID VertexID) const;


	//////////////////////////////////////////////////////////////////////
	// Polygon operations

	/** Return reference to an array of triangle IDs which comprise this polygon */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetPolygonTriangles(FPolygonID PolygonID, TArray<FTriangleID>& OutTriangleIDs) const;

	/** Return the number of triangles which comprise this polygon */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumPolygonTriangles(FPolygonID PolygonID) const;

	/** Returns reference to an array of VertexInstance IDs forming the perimeter of this polygon */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetPolygonVertexInstances(FPolygonID PolygonID, TArray<FVertexInstanceID>& OutVertexInstanceIDs) const;

	/** Returns the number of vertices this polygon has */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumPolygonVertices(FPolygonID PolygonID) const;

	/** Returns the vertices which form the polygon perimeter */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetPolygonVertices(FPolygonID PolygonID, TArray<FVertexID>& OutVertexIDs) const;

	/** Returns the edges which form the polygon perimeter */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetPolygonPerimeterEdges(FPolygonID PolygonID, TArray<FEdgeID>& OutEdgeIDs) const;

	/** Populate the provided array with a list of edges which are internal to the polygon, i.e. those which separate
	    constituent triangles. */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetPolygonInternalEdges(FPolygonID PolygonID, TArray<FEdgeID>& OutEdgeIDs) const;

	/** Return the number of internal edges in this polygon */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumPolygonInternalEdges(FPolygonID PolygonID) const;

	/** Populates the passed array with adjacent polygons */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetPolygonAdjacentPolygons(FPolygonID PolygonID, TArray<FPolygonID>& OutPolygonIDs) const;

	/** Return the polygon group associated with a polygon */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API FPolygonGroupID GetPolygonPolygonGroup(FPolygonID PolygonID) const;

	/** Return the vertex instance which corresponds to the given vertex on the given polygon, or INDEX_NONE */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API FVertexInstanceID GetVertexInstanceForPolygonVertex(FPolygonID PolygonID, FVertexID VertexID) const;

	/** Set the vertex instance at the given index around the polygon to the new value */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void SetPolygonVertexInstances(FPolygonID PolygonID, const TArray<FVertexInstanceID>& VertexInstanceIDs);

	/** Sets the polygon group associated with a polygon */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void SetPolygonPolygonGroup(FPolygonID PolygonID, FPolygonGroupID PolygonGroupID);

	/** Reverse the winding order of the vertices of this polygon */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void ReversePolygonFacing(FPolygonID PolygonID);

	/** Generates triangles and internal edges for the given polygon */
	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	MESHDESCRIPTION_API void ComputePolygonTriangulation(FPolygonID PolygonID);


	//////////////////////////////////////////////////////////////////////
	// Polygon group operations

	/** Returns the polygons associated with the given polygon group */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API void GetPolygonGroupPolygons(FPolygonGroupID PolygonGroupID, TArray<FPolygonID>& OutPolygonIDs) const;

	/** Returns the number of polygons in this polygon group */
	UFUNCTION(BlueprintPure, Category="MeshDescription")
	MESHDESCRIPTION_API int32 GetNumPolygonGroupPolygons(FPolygonGroupID PolygonGroupID) const;

protected:
	FMeshDescription OwnedMeshDescription;
	TUniquePtr<FMeshAttributes> RequiredAttributes;
};
