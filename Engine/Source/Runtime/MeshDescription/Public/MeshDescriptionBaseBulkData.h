// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Templates/PimplPtr.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MeshDescriptionBaseBulkData.generated.h"

class FArchive;
class UMeshDescriptionBase;
struct FMeshDescriptionBulkData;


/**
 * UObject wrapper for FMeshDescriptionBulkData
 */
UCLASS(MinimalAPI)
class UMeshDescriptionBaseBulkData : public UObject
{
	GENERATED_BODY()

public:
	MESHDESCRIPTION_API UMeshDescriptionBaseBulkData();
	MESHDESCRIPTION_API virtual void Serialize(FArchive& Ar) override;
	MESHDESCRIPTION_API virtual bool IsEditorOnly() const override;
	MESHDESCRIPTION_API virtual bool NeedsLoadForClient() const override;
	MESHDESCRIPTION_API virtual bool NeedsLoadForServer() const override;
	MESHDESCRIPTION_API virtual bool NeedsLoadForEditorGame() const override;

#if WITH_EDITORONLY_DATA

	MESHDESCRIPTION_API void Empty();
	MESHDESCRIPTION_API UMeshDescriptionBase* CreateMeshDescription();
	MESHDESCRIPTION_API UMeshDescriptionBase* GetMeshDescription() const;
	MESHDESCRIPTION_API bool HasCachedMeshDescription() const;
	MESHDESCRIPTION_API bool CacheMeshDescription();
	MESHDESCRIPTION_API void CommitMeshDescription(bool bUseHashAsGuid);
	MESHDESCRIPTION_API void RemoveMeshDescription();
	MESHDESCRIPTION_API bool IsBulkDataValid() const;

	MESHDESCRIPTION_API const FMeshDescriptionBulkData& GetBulkData() const;
	MESHDESCRIPTION_API FMeshDescriptionBulkData& GetBulkData();

protected:
	TPimplPtr<FMeshDescriptionBulkData> BulkData;

	UPROPERTY(Transient, Instanced)
	TObjectPtr<UMeshDescriptionBase> PreallocatedMeshDescription;

	UPROPERTY(Transient)
	TObjectPtr<UMeshDescriptionBase> MeshDescription;

#endif //WITH_EDITORONLY_DATA
};
