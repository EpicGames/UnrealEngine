// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshTypes.h"

/**
 * This defines the container used to hold mesh elements.
 * Its important properties are that it acts as an associative container (i.e. an element can be obtained from
 * a given index), and that insert/delete/find are cheap.
 * The current implementation is as a TSparseArray, but we abstract it so that this can be changed later if
 * required, e.g. a TMap might be desirable if we wished to maintain unique indices for the lifetime of the container.
 */
template <typename ElementType>
class TMeshElementArrayBase
{
public:

	/**
	 * Custom serialization for TMeshElementArrayBase.
	 * The default TSparseArray serialization also compacts all the elements, removing the gaps and changing the indices.
	 * The indices are significant in editable meshes, hence this is a custom serializer which preserves them.
	 */
	friend FArchive& operator<<( FArchive& Ar, TMeshElementArrayBase& Array )
	{
		Array.Container.CountBytes( Ar );

		if( Ar.IsLoading() )
		{
			// Load array
			TBitArray<> AllocatedIndices;
			Ar << AllocatedIndices;

			Array.Container.Empty( AllocatedIndices.Num() );
			for( auto It = TConstSetBitIterator<>( AllocatedIndices ); It; ++It )
			{
				Array.Container.Insert( It.GetIndex(), ElementType() );
				Ar << Array.Container[ It.GetIndex() ];
			}
		}
		else
		{
			// Save array
			const int32 MaxIndex = Array.Container.GetMaxIndex();

			// We have to build the TBitArray representing allocated indices by hand, as we don't have access to it from outside TSparseArray.
			// @todo core: consider replacing TSparseArray serialization with this format.
			TBitArray<> AllocatedIndices( false, MaxIndex );
			int32 MaxAllocatedIndex = 0;
			
			for( int32 Index = 0; Index < MaxIndex; ++Index )
			{
				if( Array.Container.IsAllocated( Index ) )
				{
					AllocatedIndices[ Index ] = true;
					MaxAllocatedIndex = Index;
				}
			}

			// Cut off the trailing unallocated indices so that they are not serialized
			AllocatedIndices.SetNumUninitialized(MaxAllocatedIndex + 1);

			Ar << AllocatedIndices;

			for( auto It = Array.Container.CreateIterator(); It; ++It )
			{
				Ar << *It;
			}
		}

		return Ar;
	}

	/** Compacts elements and returns a remapping table */
	void Compact( TSparseArray<int32>& OutIndexRemap );

	/** Remaps elements according to the passed remapping table */
	void Remap( const TSparseArray<int32>& IndexRemap );


protected:

	/** The actual container, represented by a sparse array */
	TSparseArray<ElementType> Container;
};


/**
 * We prefer to access elements of the container via strongly-typed IDs.
 * This derived class imposes this type safety.
 */
template <typename ElementType, typename ElementIDType>
class TMeshElementArray final : public TMeshElementArrayBase<ElementType>
{
	static_assert( TIsDerivedFrom<ElementIDType, FElementID>::IsDerived, "ElementIDType must be derived from FElementID" );

	using TMeshElementArrayBase<ElementType>::Container;

public:

	/** Resets the container, optionally reserving space for elements to be added */
	inline void Reset( const int32 Elements = 0 )
	{
		Container.Reset();
		Container.Reserve( Elements );
	}

	/** Reserves space for the specified total number of elements */
	inline void Reserve( const int32 Elements ) { Container.Reserve( Elements ); }

	/** Add a new element at the next available index, and return the new ID */
	inline ElementIDType Add() { return ElementIDType( Container.Add( ElementType() ) ); }

	/** Add the provided element at the next available index, and return the new ID */
	inline ElementIDType Add( typename TTypeTraits<ElementType>::ConstInitType Element ) { return ElementIDType( Container.Add( Element ) ); }

	/** Add the provided element at the next available index, and return the ID */
	inline ElementIDType Add( ElementType&& Element ) { return ElementIDType( Container.Add( Forward<ElementType>( Element ) ) ); }

	/** Inserts a new element with the given ID */
	inline ElementType& Insert( const ElementIDType ID )
	{
		Container.Insert( ID.GetValue(), ElementType() );
		return Container[ ID.GetValue() ];
	}

	/** Inserts the provided element with the given ID */
	inline ElementType& Insert( const ElementIDType ID, typename TTypeTraits<ElementType>::ConstInitType Element )
	{
		Container.Insert( ID.GetValue(), Element );
		return Container[ ID.GetValue() ];
	}

	/** Inserts the provided element with the given ID */
	inline ElementType& Insert( const ElementIDType ID, ElementType&& Element )
	{
		Container.Insert( ID.GetValue(), Forward<ElementType>( Element ) );
		return Container[ ID.GetValue() ];
	}

	/** Removes the element with the given ID */
	inline void Remove( const ElementIDType ID )
	{
		checkSlow( Container.IsAllocated( ID.GetValue() ) );
		Container.RemoveAt( ID.GetValue() );
	}

	/** Returns the element with the given ID */
	inline ElementType& operator[]( const ElementIDType ID )
	{
		checkSlow( Container.IsAllocated( ID.GetValue() ) );
		return Container[ ID.GetValue() ];
	}

	inline const ElementType& operator[]( const ElementIDType ID ) const
	{
		checkSlow( Container.IsAllocated( ID.GetValue() ) );
		return Container[ ID.GetValue() ];
	}

	/** Returns the number of elements in the container */
	inline int32 Num() const { return Container.Num(); }

	/** Returns the index after the last valid element */
	inline int32 GetArraySize() const { return Container.GetMaxIndex(); }

	/** Returns the first valid ID */
	inline ElementIDType GetFirstValidID() const
	{
		return Container.Num() > 0 ?
			ElementIDType( typename TSparseArray<ElementType>::TConstIterator( Container ).GetIndex() ) :
			INDEX_NONE;
	}

	/** Returns whether the given ID is valid or not */
	inline bool IsValid( const ElementIDType ID ) const
	{
		return ID.GetValue() >= 0 && ID.GetValue() < Container.GetMaxIndex() && Container.IsAllocated( ID.GetValue() );
	}

	/** Serializer */
	inline friend FArchive& operator<<( FArchive& Ar, TMeshElementArray& Array )
	{
		Ar << static_cast<TMeshElementArrayBase<ElementType>&>( Array );
		return Ar;
	}

	/**
	 * This is a special type of iterator which returns successive IDs of valid elements, rather than
	 * the elements themselves.
	 * It is designed to be used with a range-for:
	 *
	 *     for( const FVertexID VertexID : GetVertices().GetElementIDs() )
	 *     {
	 *         DoSomethingWith( VertexID );
	 *     }
	 */
	class TElementIDs
	{
	public:

		explicit inline TElementIDs( const TSparseArray<ElementType>& InArray )
			: Array( InArray )
		{}

		class TConstIterator
		{
		public:

			explicit inline TConstIterator( typename TSparseArray<ElementType>::TConstIterator&& It )
				: Iterator( MoveTemp( It ) )
			{}

			inline TConstIterator& operator++()
			{
				++Iterator;
				return *this;
			}

			inline ElementIDType operator*() const
			{
				return Iterator ? ElementIDType( Iterator.GetIndex() ) : INDEX_NONE;
			}

			friend inline bool operator==( const TConstIterator& Lhs, const TConstIterator& Rhs )
			{
				return Lhs.Iterator == Rhs.Iterator;
			}

			friend inline bool operator!=( const TConstIterator& Lhs, const TConstIterator& Rhs )
			{
				return Lhs.Iterator != Rhs.Iterator;
			}

		private:

			typename TSparseArray<ElementType>::TConstIterator Iterator;
		};

		inline TConstIterator CreateConstIterator() const
		{
			return TConstIterator( typename TSparseArray<ElementType>::TConstIterator( Array ) );
		}

	public:

		inline TConstIterator begin() const
		{
			return TConstIterator( Array.begin() );
		}

		inline TConstIterator end() const
		{
			return TConstIterator( Array.end() );
		}

	private:

		const TSparseArray<ElementType>& Array;
	};

	/** Return iterable proxy object from container */
	TElementIDs inline GetElementIDs() const { return TElementIDs( Container ); }
};


template <typename ElementType>
void TMeshElementArrayBase<ElementType>::Compact( TSparseArray<int32>& OutIndexRemap )
{
	TSparseArray<ElementType> NewContainer;
	NewContainer.Reserve( Container.Num() );

	OutIndexRemap.Empty( Container.GetMaxIndex() );

	// Add valid elements into a new contiguous sparse array.  Note non-const iterator so we can move elements.
	for( typename TSparseArray<ElementType>::TIterator It( Container ); It; ++It )
	{
		const int32 OldElementIndex = It.GetIndex();

		// @todo mesheditor: implement TSparseArray::Add( ElementType&& ) to save this obscure approach
		const int32 NewElementIndex = NewContainer.Add( ElementType() );
		NewContainer[ NewElementIndex ] = MoveTemp( *It );

		// Provide an O(1) lookup from old index to new index, used when patching up vertex references afterwards
		OutIndexRemap.Insert( OldElementIndex, NewElementIndex );
	}

	Container = MoveTemp( NewContainer );
}


template <typename ElementType>
void TMeshElementArrayBase<ElementType>::Remap( const TSparseArray<int32>& IndexRemap )
{
	TSparseArray<ElementType> NewContainer;
	NewContainer.Reserve( IndexRemap.GetMaxIndex() );

	// Add valid elements into a new contiguous sparse array.  Note non-const iterator so we can move elements.
	for( typename TSparseArray<ElementType>::TIterator It( Container ); It; ++It )
	{
		const int32 OldElementIndex = It.GetIndex();

		check( IndexRemap.IsAllocated( OldElementIndex ) );
		const int32 NewElementIndex = IndexRemap[ OldElementIndex ];

		// @todo mesheditor: implement TSparseArray::Insert( ElementType&& ) to save this obscure approach
		NewContainer.Insert( NewElementIndex, ElementType() );
		NewContainer[ NewElementIndex ] = MoveTemp( *It );
	}

	Container = MoveTemp( NewContainer );
}
