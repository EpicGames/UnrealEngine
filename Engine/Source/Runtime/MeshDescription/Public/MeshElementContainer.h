// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "MeshAttributeArray.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"


/**
 * Class representing a collection of mesh elements, such as vertices or triangles.
 * It has two purposes:
 *  1) To generate new element IDs on demand, and recycle those which have been discarded.
 *  2) To hold a map of attributes for a given element type and their values.
 */
class FMeshElementContainer
{
public:
	FMeshElementContainer() = default;
	FMeshElementContainer(const FMeshElementContainer&) = default;
	FMeshElementContainer& operator=(const FMeshElementContainer&) = default;

	/** Move constructor which ensures that NumHoles is set correctly in the moved object */
	FMeshElementContainer(FMeshElementContainer&& Other)
	{
		BitArray = MoveTemp(Other.BitArray);
		Attributes = MoveTemp(Other.Attributes);
		NumHoles = Other.NumHoles;
		Other.NumHoles = 0;
	}

	/** Move assignment operator which ensures that NumHoles is set correctly in the moved object */
	FMeshElementContainer& operator=(FMeshElementContainer&& Other)
	{
		if (this != &Other)
		{
			BitArray = MoveTemp(Other.BitArray);
			Attributes = MoveTemp(Other.Attributes);
			NumHoles = Other.NumHoles;
			Other.NumHoles = 0;
		}
		return *this;
	}

	/** Resets the container, optionally reserving space for elements to be added */
	void Reset(const int32 Elements = 0)
	{
		BitArray.Empty(Elements);
		Attributes.Initialize(0);
		NumHoles = 0;
	}

	/** Reserves space for the specified total number of elements */
	void Reserve(const int32 Elements) { BitArray.Reserve(Elements); }

	/** Add a new element at the next available index, and return the new ID */
	int32 Add()
	{
		if (NumHoles > 0)
		{
			// If there are holes, use those up first.
			const int32 Index = BitArray.FindAndSetFirstZeroBit();
			check(Index != INDEX_NONE);
			NumHoles--;
			return Index;
		}
		else
		{
			// Otherwise add a new element, and insert a corresponding attribute slot.
			const int32 Index = BitArray.Add(true);
			Attributes.Insert(Index);
			return Index;
		}
	}

	/** Inserts a new element with the given index */
	void Insert(const int32 Index)
	{
		checkSlow(Index >= 0);
		if (Index >= BitArray.Num())
		{
			// If the index is beyond the current bit array size, create a new one, padded out with zeroes.
			const int32 FirstNewIndex = BitArray.Num();
			const int32 NumNewHoles = Index - FirstNewIndex;
			BitArray.SetNumUninitialized(Index + 1);
			BitArray.SetRange(FirstNewIndex, NumNewHoles, false);
			BitArray[Index] = true;
			NumHoles += NumNewHoles;
			Attributes.Insert(Index);
		}
		else
		{
			// Can't insert an index over an existing one.
			// If we get here, we assume this is a hole, so decrement the number of holes.
			checkSlow(!BitArray[Index]);
			BitArray[Index] = true;
			NumHoles--;
		}
	}

	/** Removes the element with the given ID */
	void Remove(const int32 Index)
	{
		// We can't remove an element which is already a hole.
		checkSlow(BitArray[Index]);
		BitArray[Index] = false;
		NumHoles++;
		Attributes.Remove(Index);
	}

	/** Returns the number of elements in the container */
	int32 Num() const { return BitArray.Num() - NumHoles; }

	/** Returns the index after the last valid element */
	int32 GetArraySize() const { return BitArray.Num(); }

	/** Returns the first valid ID */
	int32 GetFirstValidID() const
	{
		return BitArray.Find(true);
	}

	/** Returns whether the given ID is valid or not */
	bool IsValid(const int32 Index) const
	{
		return Index >= 0 && Index < BitArray.Num() && BitArray[Index];
	}

	/** Accessor for attributes */
	inline FAttributesSetBase& GetAttributes() { return Attributes; }
	inline const FAttributesSetBase& GetAttributes() const { return Attributes; }

	/** Compacts elements and returns a remapping table */
	MESHDESCRIPTION_API void Compact(TSparseArray<int32>& OutIndexRemap);

	/** Remaps elements according to the passed remapping table */
	MESHDESCRIPTION_API void Remap(const TSparseArray<int32>& IndexRemap);

	/** Serializer */
	friend FArchive& operator<<(FArchive& Ar, FMeshElementContainer& Container)
	{
		Ar << Container.BitArray;
		// We could count the number of holes in the BitArray, but it is quicker for serialization purposes (particularly in transactions) to just store it.
		Ar << Container.NumHoles;
		Ar << Container.Attributes;
		return Ar;
	}

	/**
	 * This is a special type of iterator which returns successive IDs of valid elements, rather than
	 * the elements themselves.
	 * It is designed to be used with a range-for:
	 *
	 *     for (const int32 VertexIndex : GetVertices().GetElementIDs())
	 *     {
	 *         DoSomethingWith(VertexIndex);
	 *     }
	 */
	class FElementIDs
	{
	public:
		explicit inline FElementIDs(const TBitArray<>& InArray)
			: Array(InArray)
		{}

		class FConstIterator
		{
		public:
			explicit inline FConstIterator(TConstSetBitIterator<>&& It)
				: Iterator(MoveTemp(It))
			{}

			inline FConstIterator& operator++()
			{
				++Iterator;
				return *this;
			}

			inline int32 operator*() const
			{
				return Iterator ? Iterator.GetIndex() : INDEX_NONE;
			}

			friend inline bool operator==(const FConstIterator& Lhs, const FConstIterator& Rhs)
			{
				return Lhs.Iterator == Rhs.Iterator;
			}

			friend inline bool operator!=(const FConstIterator& Lhs, const FConstIterator& Rhs)
			{
				return Lhs.Iterator != Rhs.Iterator;
			}

		private:
			TConstSetBitIterator<> Iterator;
		};

		inline FConstIterator CreateConstIterator() const
		{
			return FConstIterator(TConstSetBitIterator<>(Array));
		}

	public:
		inline FConstIterator begin() const
		{
			return FConstIterator(TConstSetBitIterator<>(Array));
		}

		inline FConstIterator end() const
		{
			return FConstIterator(TConstSetBitIterator<>(Array, Array.Num()));
		}

	private:
		const TBitArray<>& Array;
	};

	/** Return iterable proxy object from container */
	FElementIDs inline GetElementIDs() const { return FElementIDs(BitArray); }

protected:
	/** A bit array of all indices currently in use */
	TBitArray<> BitArray;

	/** Attributes associated with this element container */
	FAttributesSetBase Attributes;

	/** A count of the number of unused indices in the bit array. IMPORTANT: This must always correspond to the number of zeroes in the BitArray. */
	int32 NumHoles = 0;
};


/**
 * Templated specialization for type-safety.
 */
template <typename ElementIDType>
class TMeshElementContainer : public FMeshElementContainer
{
public:
	/** Add a new element at the next available index, and return the new ID */
	ElementIDType Add()
	{
		return ElementIDType(FMeshElementContainer::Add());
	}

	/** Inserts a new element with the given index */
	void Insert(const ElementIDType Index)
	{
		FMeshElementContainer::Insert(Index.GetValue());
	}

	/** Removes the element with the given ID */
	void Remove(const ElementIDType Index)
	{
		FMeshElementContainer::Remove(Index.GetValue());
	}

	/** Returns the first valid ID */
	ElementIDType GetFirstValidID() const
	{
		return ElementIDType(FMeshElementContainer::GetFirstValidID());
	}

	/** Returns whether the given ID is valid or not */
	bool IsValid(const ElementIDType Index) const
	{
		return FMeshElementContainer::IsValid(Index.GetValue());
	}

	/** Accessor for attributes */
	inline TAttributesSet<ElementIDType>& GetAttributes() { return static_cast<TAttributesSet<ElementIDType>&>(Attributes); }
	inline const TAttributesSet<ElementIDType>& GetAttributes() const { return static_cast<const TAttributesSet<ElementIDType>&>(Attributes); }

	/** Serializer */
	friend FArchive& operator<<(FArchive& Ar, TMeshElementContainer& Container)
	{
		Ar << static_cast<FMeshElementContainer&>(Container);
		return Ar;
	}

	/**
	 * This is a special type of iterator which returns successive IDs of valid elements, rather than
	 * the elements themselves.
	 * It is designed to be used with a range-for:
	 *
	 *     for (const int32 VertexIndex : GetVertices().GetElementIDs())
	 *     {
	 *         DoSomethingWith(VertexIndex);
	 *     }
	 */
	class TElementIDs
	{
	public:
		explicit inline TElementIDs(const TBitArray<>& InArray)
			: Array(InArray)
		{}

		class TConstIterator
		{
		public:
			explicit inline TConstIterator(TConstSetBitIterator<>&& It)
				: Iterator(MoveTemp(It))
			{}

			inline TConstIterator& operator++()
			{
				++Iterator;
				return *this;
			}

			inline ElementIDType operator*() const
			{
				return ElementIDType{Iterator ? Iterator.GetIndex() : INDEX_NONE};
			}

			friend inline bool operator==(const TConstIterator& Lhs, const TConstIterator& Rhs)
			{
				return Lhs.Iterator == Rhs.Iterator;
			}

			friend inline bool operator!=(const TConstIterator& Lhs, const TConstIterator& Rhs)
			{
				return Lhs.Iterator != Rhs.Iterator;
			}

		private:
			TConstSetBitIterator<> Iterator;
		};

		inline TConstIterator CreateConstIterator() const
		{
			return TConstIterator(TConstSetBitIterator<>(Array));
		}

	public:
		inline TConstIterator begin() const
		{
			return TConstIterator(TConstSetBitIterator<>(Array));
		}

		inline TConstIterator end() const
		{
			return TConstIterator(TConstSetBitIterator<>(Array, Array.Num()));
		}

	private:
		const TBitArray<>& Array;
	};

	/** Return iterable proxy object from container */
	TElementIDs inline GetElementIDs() const { return TElementIDs(BitArray); }
};


/**
 * This is a wrapper for an array of allocated MeshElementContainers.
 */
class FMeshElementChannels
{
public:
	/** Default constructor creates a single element */
	explicit FMeshElementChannels(const int32 NumberOfIndices = 1)
	{
		Channels.SetNum(NumberOfIndices);
	}

	/** Transparent access through the array */
	inline const FMeshElementContainer& Get(const int32 Index = 0) const { return Channels[Index]; }
	inline const FMeshElementContainer& operator[](const int32 Index) const { return Channels[Index]; }
	inline const FMeshElementContainer& operator*() const { return Channels[0]; }
	inline const FMeshElementContainer* operator->() const { return &Channels[0]; }
	inline FMeshElementContainer& Get(const int32 Index = 0) { return Channels[Index]; }
	inline FMeshElementContainer& operator[](const int32 Index) { return Channels[Index]; }
	inline FMeshElementContainer& operator*() { return Channels[0]; }
	inline FMeshElementContainer* operator->() { return &Channels[0]; }

	/** Change the number of indices */
	void SetNumChannels(const int32 NumIndices) { Channels.SetNum(NumIndices); }

	/** Get the number of indices */
	int32 GetNumChannels() const { return Channels.Num(); }

	/** Resets the containers */
	void Reset()
	{
		for (FMeshElementContainer& Channel : Channels)
		{
			Channel.Reset();
		}
	}

	/** Determines whether the mesh element type is empty or not */
	bool IsEmpty() const
	{
		for (const FMeshElementContainer& Channel : Channels)
		{
			if (Channel.GetArraySize() != 0) { return false; }
		}

		return true;
	}

	/** Serializer */
	friend FArchive& operator<<(FArchive& Ar, FMeshElementChannels& ElementType)
	{
		Ar << ElementType.Channels;
		return Ar;
	}

private:
	// The usual thing is that a MeshElementType has exactly one channel, so we specially reserve a single element inline to save extra allocations
	TArray<FMeshElementContainer, TInlineAllocator<1>> Channels;
};


/**
 * This is a wrapper for a FMeshElementChannels.
 * It holds a TUniquePtr pointing to the actual FMeshElementChannels, so that its address is constant and can be cached.
 */
class FMeshElementTypeWrapper
{
public:
	/** Default constructor - construct a MeshElementType, optionally with more than one channel */
	explicit FMeshElementTypeWrapper(const int32 NumberOfChannels = 1)
	{
		check(NumberOfChannels > 0);
		Ptr = MakeUnique<FMeshElementChannels>(NumberOfChannels);
	}

	/** Copy constructor - make a deep copy of the MeshElementType through the TUniquePtr */
	FMeshElementTypeWrapper(const FMeshElementTypeWrapper& Other)
	{
		check(Other.Ptr.IsValid());
		Ptr = MakeUnique<FMeshElementChannels>(*Other.Ptr);
	}

	/** Copy assignment */
	FMeshElementTypeWrapper& operator=(const FMeshElementTypeWrapper& Other)
	{
		FMeshElementTypeWrapper Temp(Other);
		Swap(*this, Temp);
		return *this;
	}

	/** Default move constructor */
	FMeshElementTypeWrapper(FMeshElementTypeWrapper&&) = default;

	/** Default move assignment */
	FMeshElementTypeWrapper& operator=(FMeshElementTypeWrapper&&) = default;

	/** Transparent access through the TUniquePtr */
	const FMeshElementChannels* Get() const { return Ptr.Get(); }
	const FMeshElementChannels* operator->() const { return Ptr.Get(); }
	const FMeshElementChannels& operator*() const { return *Ptr; }
	FMeshElementChannels* Get() { return Ptr.Get(); }
	FMeshElementChannels* operator->() { return Ptr.Get(); }
	FMeshElementChannels& operator*() { return *Ptr; }

	/** Serializer */
	friend FArchive& operator<<(FArchive& Ar, FMeshElementTypeWrapper& Wrapper)
	{
		Ar << *Wrapper.Ptr;
		return Ar;
	}

private:
	TUniquePtr<FMeshElementChannels> Ptr;
};