// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "MeshAttributeArray.h"
#include "Misc/AssertionMacros.h"
#include "Templates/EnableIf.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"

class FMeshElementChannels;
struct FElementID;


/**
 * This is an efficient container for holding back references to mesh elements from attributes.
 * Typically this will be a one-to-many relationship, where a given key has many different references.
 *
 * Naively this would be implemented as an array of arrays, but we want to minimise allocations and fragmentation
 * as much as possible. This is achieved in two ways:
 *
 * 1) Arrays representing the list of references are consolidated into chunks of a user-specified size.
 *    This reduces the number of individual allocations which need to be made. By default, 256 keys are consolidated
 *    into a single chunk.
 *
 * 2) The arrays are lazily updated. The most expensive aspect of consolidating adjacent keys' values into a single
 *    chunk of data is having to insert new space into the middle of the chunk. If a new value can't be immediately
 *    added into a chunk without inserting, the key is marked as stale and the referencer is remembered. If a stale
 *    key is queried, the indexer will be updated first, building all stale keys by getting back references from the
 *    remembered referencers.
 *
 * There is also a way to fully rebuild an indexer, which will perform optimally few allocations, and collect all
 * back references at once. It may sometimes be desirable to suspend an indexer, build a mesh description with indexing
 * suspended, and then perform an indexer rebuild for additional speed and allocation efficiency.
 *
 * Some element relationships don't lend themselves to chunking, for example polygon groups to triangles. In this case,
 * there are generally few polygon groups compared to many triangles. Chunking can be turned off for such indexers,
 * resulting in a more simple array representation which can be arbitrarily grown without requiring elements to be inserted.
 */
class FMeshElementIndexer
{
public:
	FMeshElementIndexer() = default;
	MESHDESCRIPTION_API FMeshElementIndexer(const FMeshElementChannels* Key, const FMeshElementChannels* Referencer, FName AttributeName, int32 ReferencerChannelIndex = 0);

	// Copying not allowed as they reference elements of a MeshDescription of which they are normally a part
	FMeshElementIndexer(const FMeshElementIndexer&) = delete;
	FMeshElementIndexer& operator=(const FMeshElementIndexer&) = delete;

	// Moving OK
	FMeshElementIndexer(FMeshElementIndexer&&) = default;
	FMeshElementIndexer& operator=(FMeshElementIndexer&&) = default;

	/**
	 * Set the parameters of this mesh element indexer
	 */
	MESHDESCRIPTION_API void Set(const FMeshElementChannels* Key, const FMeshElementChannels* Referencer, FName AttributeName, int32 ReferencerChannelIndex = 0);

	/**
	 * Sets the initial/expected number of references per key
	 */
	void SetInitialNumReferences(int InInitialSize) { InitialSize = InInitialSize; }

	/**
	 * Sets the chunk size for the indexer, for optimization purposes.
	 * The default chunk size is 256.
	 */
	void SetChunkSize(int InChunkSize)
	{
		ChunkBits = FMath::CeilLogTwo(InChunkSize);
		ChunkMask = (1U << ChunkBits) - 1;
	}

	/**
	 * Configures the indexer not to use chunks.
	 * This is desirable if there are very few keys with a lot of references, e.g. polygon groups to polygons
	 */
	void SetUnchunked()
	{
		SetChunkSize(1);
		check(ChunkBits == 0);
		check(ChunkMask == 0);
	}

	/**
	 * Resets the indexer
	 */
	MESHDESCRIPTION_API void Reset();

	/**
	 * Retrieve all referencer indices whose named attribute refers to KeyIndex.
	 * Non-const because finding a stale index might cause a rebuild.
	 */
	TArrayView<const int32> Find(int32 KeyIndex, int32 KeyChannelIndex = 0)
	{
		check(!bSuspended);
		ConditionalBuild(KeyIndex, KeyChannelIndex);
		int32 Chunk = GetChunkForKey(KeyIndex);
		int32 ChunkElement = GetChunkElementForKey(KeyIndex);

		check(KeyChannelIndex < PerChannelIndices.Num());
		if (PerChannelIndices[KeyChannelIndex].Chunks.Num() == 0)
		{
			return TArrayView<const int32>();
		}
		return PerChannelIndices[KeyChannelIndex].Chunks[Chunk].Get(ChunkElement);
	}


	template <typename ElementIDType>
	TArrayView<const ElementIDType> Find(int32 KeyIndex, int32 KeyChannelIndex = 0)
	{
		check(!bSuspended);
		check(KeyIndex != INDEX_NONE);
		ConditionalBuild(KeyIndex, KeyChannelIndex);
		int32 Chunk = GetChunkForKey(KeyIndex);
		int32 ChunkElement = GetChunkElementForKey(KeyIndex);

		// If the key hasn't yet had a chunk created for it, assume that nothing has yet referenced it, and return an empty array.
		if (Chunk >= PerChannelIndices[KeyChannelIndex].Chunks.Num())
		{
			return TArrayView<const ElementIDType>();
		}
		return PerChannelIndices[KeyChannelIndex].Chunks[Chunk].Get<ElementIDType>(ChunkElement);
	}


	/**
	 * Remove the specified key from the indexer
	 */
	MESHDESCRIPTION_API void RemoveKey(int32 KeyIndex, int32 KeyChannelIndex = 0);

	/**
	 * Remove reference from key.
	 * This will take immediate effect.
	 */
	MESHDESCRIPTION_API void RemoveReferenceFromKey(int32 KeyIndex, int32 ReferenceIndex, int32 KeyChannelIndex = 0);

	/**
	 * Add a reference to a key.
	 * This will be queued and take effect in batch when an affected key is looked up.
	 * This is preferable to regenerating the index entirely.
	 */
	MESHDESCRIPTION_API void AddReferenceToKey(int32 KeyIndex, int32 ReferenceIndex, int32 KeyChannelIndex = 0);

	/**
	 * Suspend indexing until the next rebuild
	 */
	void Suspend() { bSuspended = true; }

	/**
	 * Resume indexing and mark indexer as stale, but do not force an immediate rebuild
	 */
	void Resume()
	{
		bSuspended = false;
		bEverythingStale = true;
	}

	/**
	 * Force the indexer to be completely rebuilt.
	 */
	MESHDESCRIPTION_API void ForceRebuild();

	/**
	 * Build any stale indices
	 */
	MESHDESCRIPTION_API void Build();

private:
	// Ensure that the right number of per channel indices are allocated
	MESHDESCRIPTION_API void UpdatePerChannelIndices();

	// Incrementally builds the index based on the lists of stale keys and referencers
	MESHDESCRIPTION_API void BuildIndex(int32 Index);

	// Does a full rebuild of the index from scratch, directly from referencing attributes
	MESHDESCRIPTION_API void RebuildIndex(int32 Index);

	// Performs an incremental build if the specified key is stale
	MESHDESCRIPTION_API void ConditionalBuild(int32 KeyIndex, int32 KeyChannelIndex);

	// Add a reference to a key in unchunked mode
	MESHDESCRIPTION_API void AddUnchunked(int32 KeyIndex, int32 ReferenceIndex, int32 KeyChannelIndex = 0);

	// Return the chunk index relating to this key index
	int32 GetChunkForKey(int32 ElementIndex) const
	{
		checkSlow(ElementIndex != INDEX_NONE);
		return ElementIndex >> ChunkBits;
	}

	// Return the chunk element index relating to this key index
	int32 GetChunkElementForKey(int32 ElementIndex) const
	{
		checkSlow(ElementIndex != INDEX_NONE);
		return ElementIndex & ChunkMask;
	}

	// Return the currently configured chunk size
	int32 GetChunkSize() const { return (1 << ChunkBits); }

	// Return the number of chunks required to accommodate the specified number of keys
	int32 GetNumChunksForKeys(int32 NumElements) const { return (NumElements + ChunkMask) >> ChunkBits; }


	/**
	 * We are modelling a one-to-many relationship, where a single key yields multiple indices,
	 * e.g. obtaining all the vertex instances which reference a given vertex ID (the key).
	 *
	 * We consolidate the data into chunks, each representing a set number of keys.
	 * This provides a balance between the number of separate allocations, and the cost of inserting a new value at a key.
	 */
	struct FChunk
	{
		FChunk(int32 InitialSize, int32 ChunkSize)
		{
			// Allocate the block which will contain the actual referencing elements
			Data.SetNumUninitialized(ChunkSize * InitialSize, EAllowShrinking::No);

			// Initialize indexing structures
			StartIndex.SetNumUninitialized(ChunkSize);
			Count.SetNumUninitialized(ChunkSize);
			MaxCount.SetNumUninitialized(ChunkSize);

			int32 N = 0;
			for (int I = 0; I < ChunkSize; I++)
			{
				StartIndex[I] = N;
				N += InitialSize;
			}

			for (int I = 0; I < ChunkSize; I++)
			{
				MaxCount[I] = InitialSize;
			}

			FMemory::Memzero(Count.GetData(), sizeof(int32) * ChunkSize);
		}

		/** Add a value to the chunk at the given index */
		bool Add(int32 Index, int32 Value);

		/** Add a value to the chunk at the given index, without inserting it in sorted order */
		bool AddUnsorted(int32 Index, int32 Value);

		/** Remove a value from the chunk at the given index */
		bool Remove(int32 Index, int32 Value);

		/** Determine whether a value exists at the given index */
		bool Contains(int32 Index, int32 Value) const;

		/** Sort values at the given index */
		void Sort(int32 Index);

		/** Get the range of data corresponding to this element */
		TArrayView<const int32> Get(int32 Index) const
		{
			// Build array view for the portion of the data array corresponding to this element
			const int32* Ptr = Data.GetData() + StartIndex[Index];
			return TArrayView<const int32>(Ptr, Count[Index]);
		}

		template <typename ElementIDType, typename TEnableIf<TIsDerivedFrom<ElementIDType, FElementID>::Value, int>::Type = 0>
		TArrayView<const ElementIDType> Get(int32 Index) const
		{
			// Build array view for the portion of the data array corresponding to this element
			static_assert(sizeof(ElementIDType) == sizeof(int32), "ElementIDType must be 32 bits long");
			const ElementIDType* Ptr = reinterpret_cast<const ElementIDType*>(Data.GetData()) + StartIndex[Index];
			return TArrayView<const ElementIDType>(Ptr, Count[Index]);
		}

		/** All the data for this chunk */
		TArray<int32> Data;

		/**
		 * This is how we determine the location and size of a given key's data in the consolidated data.
		 * Note: deleting a value will not shrink the data.
		 */
		TArray<int32, TInlineAllocator<1>> StartIndex;
		TArray<int32, TInlineAllocator<1>> Count;
		TArray<int32, TInlineAllocator<1>> MaxCount;
	};

	struct FIndexPerChannel
	{
		/** List of chunks for the keyed data */
		TArray<FChunk> Chunks;

		/** List of keys which need to be rebuilt */
		TSet<int32> StaleKeyIndices;

		/** List of referencers which should be examined to rebuild the index */
		TSet<int32> StaleReferencerIndices;
	};

	TArray<FIndexPerChannel> PerChannelIndices;

	/** The element type used as the key for this index */
	const FMeshElementChannels* ReferencedElementType;

	/** The attribute ref used to reference the above key */
	TMeshAttributesRef<int, TArrayView<const int32>> ReferencingAttributeRef;

	/** The initial size for key chunks */
	int32 InitialSize = 1;

	/** Chunk size */
	uint32 ChunkBits = 8;
	uint32 ChunkMask = (1U << ChunkBits) - 1;

	/** Whether indexing is temporarily suspended */
	bool bSuspended = false;

	/** True if the entire data needs to be rebuilt */
	bool bEverythingStale = true;
};

