// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshTypes.h"

/**
 * This is a structure which holds the ID remappings returned by a Compact operation, or passed to a Remap operation.
 */
struct FElementIDRemappings
{
	TSparseArray<int32> NewVertexIndexLookup;
	TSparseArray<int32> NewVertexInstanceIndexLookup;
	TSparseArray<int32> NewEdgeIndexLookup;
	TSparseArray<int32> NewTriangleIndexLookup;
	TSparseArray<int32> NewUVIndexLookup;
	TSparseArray<int32> NewPolygonIndexLookup;
	TSparseArray<int32> NewPolygonGroupIndexLookup;

	FVertexID GetRemappedVertexID( FVertexID VertexID ) const
	{
		checkSlow( NewVertexIndexLookup.IsAllocated( VertexID.GetValue() ) );
		return FVertexID( NewVertexIndexLookup[ VertexID.GetValue() ] );
	}

	FVertexInstanceID GetRemappedVertexInstanceID( FVertexInstanceID VertexInstanceID ) const
	{
		checkSlow( NewVertexInstanceIndexLookup.IsAllocated( VertexInstanceID.GetValue() ) );
		return FVertexInstanceID( NewVertexInstanceIndexLookup[ VertexInstanceID.GetValue() ] );
	}

	FEdgeID GetRemappedEdgeID( FEdgeID EdgeID ) const
	{
		checkSlow( NewEdgeIndexLookup.IsAllocated( EdgeID.GetValue() ) );
		return FEdgeID( NewEdgeIndexLookup[ EdgeID.GetValue() ] );
	}

	FUVID GetRemappedUVID( FUVID UVID ) const
	{
		checkSlow( NewUVIndexLookup.IsAllocated( UVID.GetValue() ) );
		return FUVID( NewUVIndexLookup[ UVID.GetValue() ] );
	}

	FTriangleID GetRemappedTriangleID( FTriangleID TriangleID ) const
	{
		checkSlow( NewTriangleIndexLookup.IsAllocated( TriangleID.GetValue() ) );
		return FTriangleID( NewTriangleIndexLookup[ TriangleID.GetValue() ] );
	}

	FPolygonID GetRemappedPolygonID( FPolygonID PolygonID ) const
	{
		checkSlow( NewPolygonIndexLookup.IsAllocated( PolygonID.GetValue() ) );
		return FPolygonID( NewPolygonIndexLookup[ PolygonID.GetValue() ] );
	}

	FPolygonGroupID GetRemappedPolygonGroupID( FPolygonGroupID PolygonGroupID ) const
	{
		checkSlow( NewPolygonGroupIndexLookup.IsAllocated( PolygonGroupID.GetValue() ) );
		return FPolygonGroupID( NewPolygonGroupIndexLookup[ PolygonGroupID.GetValue() ] );
	}
};

