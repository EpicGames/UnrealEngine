// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Serialization/Archive.h"
#include "Templates/TypeHash.h"
#include "UObject/ObjectMacros.h"

#include "MeshTypes.generated.h"

MESHDESCRIPTION_API DECLARE_LOG_CATEGORY_EXTERN( LogMeshDescription, Log, All );


// @todo mesheditor: Need comments

USTRUCT( BlueprintType )
struct FElementID	// @todo mesheditor script: BP doesn't have name spaces, so we might need a more specific display name, or just rename our various types
{
	GENERATED_BODY()

	FElementID()
		: IDValue(INDEX_NONE)
	{
	}

	FElementID( const int32 InitIDValue )
		: IDValue( InitIDValue )
	{
	}

	inline int32 GetValue() const
	{
		return IDValue;
	}

	inline operator int32() const
	{
		return IDValue;
	}

	inline bool operator==( const FElementID& Other ) const
	{
		return IDValue == Other.IDValue;
	}

	inline bool operator==( const int32 Other ) const
	{
		return IDValue == Other;
	}

	inline bool operator!=( const FElementID& Other ) const
	{
		return IDValue != Other.IDValue;
	}

	inline bool operator!=( const int32 Other ) const
	{
		return IDValue != Other;
	}

	FString ToString() const
	{
		return ( IDValue == INDEX_NONE ) ? TEXT( "Invalid" ) : FString::Printf( TEXT( "%d" ), IDValue );
	}

	friend FArchive& operator<<( FArchive& Ar, FElementID& Element )
	{
		Ar << Element.IDValue;
		return Ar;
	}

	/** Invalid element ID */
	UE_DEPRECATED(4.26, "Please use INDEX_NONE as an invalid ID.")
	MESHDESCRIPTION_API static const FElementID Invalid;

protected:

	/** The actual mesh element index this ID represents.  Read-only. */
	UPROPERTY( BlueprintReadOnly, Category="Editable Mesh" )
	int32 IDValue;
};


USTRUCT( BlueprintType )
struct FVertexID : public FElementID
{
	GENERATED_BODY()

	FVertexID()
	{
	}

	FVertexID( const FElementID InitElementID )
		: FElementID( InitElementID.GetValue() )
	{
	}

	FVertexID( const int32 InitIDValue )
		: FElementID( InitIDValue )
	{
	}

	inline friend uint32 GetTypeHash( const FVertexID& Other )
	{
		return GetTypeHash( Other.IDValue );
	}

	/** Invalid vertex ID */
	UE_DEPRECATED(4.26, "Please use INDEX_NONE as an invalid ID.")
	MESHDESCRIPTION_API static const FVertexID Invalid;
};


USTRUCT( BlueprintType )
struct FVertexInstanceID : public FElementID
{
	GENERATED_BODY()

	FVertexInstanceID()
	{
	}

	FVertexInstanceID( const FElementID InitElementID )
		: FElementID( InitElementID.GetValue() )
	{
	}

	FVertexInstanceID( const int32 InitIDValue )
		: FElementID( InitIDValue )
	{
	}

	inline friend uint32 GetTypeHash( const FVertexInstanceID& Other )
	{
		return GetTypeHash( Other.IDValue );
	}

	/** Invalid rendering vertex ID */
	UE_DEPRECATED(4.26, "Please use INDEX_NONE as an invalid ID.")
	MESHDESCRIPTION_API static const FVertexInstanceID Invalid;
};


USTRUCT( BlueprintType )
struct FEdgeID : public FElementID
{
	GENERATED_BODY()

	FEdgeID()
	{
	}

	FEdgeID( const FElementID InitElementID )
		: FElementID( InitElementID.GetValue() )
	{
	}

	FEdgeID( const int32 InitIDValue )
		: FElementID( InitIDValue )
	{
	}

	inline friend uint32 GetTypeHash( const FEdgeID& Other )
	{
		return GetTypeHash( Other.IDValue );
	}

	/** Invalid edge ID */
	UE_DEPRECATED(4.26, "Please use INDEX_NONE as an invalid ID.")
	MESHDESCRIPTION_API static const FEdgeID Invalid;
};


USTRUCT( BlueprintType )
struct FUVID : public FElementID
{
	GENERATED_BODY()

	FUVID()
	{
	}

	FUVID( const FElementID InitElementID )
		: FElementID( InitElementID.GetValue() )
	{
	}

	FUVID( const int32 InitIDValue )
		: FElementID( InitIDValue )
	{
	}

	inline friend uint32 GetTypeHash( const FUVID& Other )
	{
		return GetTypeHash( Other.IDValue );
	}
};


USTRUCT(BlueprintType)
struct FTriangleID : public FElementID
{
	GENERATED_BODY()

	FTriangleID()
	{
	}

	FTriangleID(const FElementID InitElementID)
		: FElementID(InitElementID.GetValue())
	{
	}

	FTriangleID( const int32 InitIDValue )
		: FElementID( InitIDValue )
	{
	}

	inline friend uint32 GetTypeHash(const FTriangleID& Other)
	{
		return GetTypeHash(Other.IDValue);
	}

	/** Invalid edge ID */
	UE_DEPRECATED(4.26, "Please use INDEX_NONE as an invalid ID.")
	MESHDESCRIPTION_API static const FTriangleID Invalid;
};


USTRUCT( BlueprintType )
struct FPolygonGroupID : public FElementID
{
	GENERATED_BODY()

	FPolygonGroupID()
	{
	}

	FPolygonGroupID( const FElementID InitElementID )
		: FElementID( InitElementID.GetValue() )
	{
	}

	FPolygonGroupID( const int32 InitIDValue )
		: FElementID( InitIDValue )
	{
	}

	inline friend uint32 GetTypeHash( const FPolygonGroupID& Other )
	{
		return GetTypeHash( Other.IDValue );
	}

	/** Invalid section ID */
	UE_DEPRECATED(4.26, "Please use INDEX_NONE as an invalid ID.")
	MESHDESCRIPTION_API static const FPolygonGroupID Invalid;
};


USTRUCT( BlueprintType )
struct FPolygonID : public FElementID
{
	GENERATED_BODY()

	FPolygonID()
	{
	}

	FPolygonID( const FElementID InitElementID )
		: FElementID( InitElementID.GetValue() )
	{
	}

	FPolygonID( const int32 InitIDValue )
		: FElementID( InitIDValue )
	{
	}

	inline friend uint32 GetTypeHash( const FPolygonID& Other )
	{
		return GetTypeHash( Other.IDValue );
	}

	/** Invalid polygon ID */
	UE_DEPRECATED(4.26, "Please use INDEX_NONE as an invalid ID.")
	MESHDESCRIPTION_API static const FPolygonID Invalid;	// @todo mesheditor script: Can we expose these to BP nicely?	Do we even need to?
};
